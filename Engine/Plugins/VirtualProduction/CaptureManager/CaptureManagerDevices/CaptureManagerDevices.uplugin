{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0.0",
	"FriendlyName": "Capture Manager Devices",
	"Description": "The Capture Manager Devices contains devices that can be used from the Capture Manager layout of the LiveLink Hub",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "MonoVideoIngestDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CPSLiveLinkDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TakeArchiveIngestDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "StereoVideoIngestDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "VideoLiveLinkDeviceCommon",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "CaptureManagerApp",
			"Enabled": true
		},
		{
			"Name": "CaptureManagerCore",
			"Enabled": true
		},
		{
			"Name": "RTSPMedia",
			"Enabled": true
		}
	]
}