// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CPSLiveLinkDevice : ModuleRules
{
	public CPSLiveLinkDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "CPSLLD";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"CaptureProtocolStack",
			"CaptureManagerCPSClient",
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Engine",
			"Json",
			"CaptureManagerMediaRW",
			"DataIngestCore",
			"IngestLiveLinkDevice",
			"LiveLinkDevice",
			"LiveLinkCapabilities",
			"Media",
			"LiveLinkHubCaptureMessaging",
			"LiveLinkHub",
			"RTSPMedia",
			"CaptureMetadataExtraction",
			"ToolWidgets",
			"Slate",
			"SlateCore",
			"CaptureManagerTakeMetadata",
			"CaptureManagerSettings",
			"CoreUObject",
			"UnrealEd"
		});

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
