// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DeviceIpAddressCustomization.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "DeviceIpAddressCustomization"

FDeviceIpAddressCustomization::FDeviceIpAddressCustomization()
	: IpAddressRange(TRange<int32>::Inclusive(0, 255))
{
}

void FDeviceIpAddressCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FDeviceIpAddressCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	DeviceIpAddressProperty = PropertyHandle;

	ChildBuilder.AddProperty(PropertyHandle)
		.CustomWidget()
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Text(this, &FDeviceIpAddressCustomization::OnGetDeviceIpAddress)
			.OnVerifyTextChanged(this, &FDeviceIpAddressCustomization::OnDeviceIpAddressVerify)
			.OnTextCommitted(this, &FDeviceIpAddressCustomization::OnDeviceIpAddressCommited)
			.SelectAllTextWhenFocused(true)
			.RevertTextOnEscape(true)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsReadOnly(this, &FDeviceIpAddressCustomization::IsReadOnly)
		];
}

TOptional<FText> FDeviceIpAddressCustomization::VerifyIpAddress(const FString& InIpAddress, TRange<int32> InIpAddressRange)
{
	if (InIpAddress.IsEmpty())
	{
		return LOCTEXT("DeviceIpAddress_Empty", "Device Ip Address must not be empty");
	}

	constexpr uint16 IPv4AddressMaxSize = 15;
	if (InIpAddress.Len() > IPv4AddressMaxSize)
	{
		return LOCTEXT("DeviceIpAddress_InvalidSize", "Device Ip Address contains too many characters");
	}

	TArray<FString> Components;
	InIpAddress.ParseIntoArray(Components, TEXT("."), false);

	if (Components.Num() != 4)
	{
		return LOCTEXT("DeviceIpAddress_InvalidFormat", "Invalid format for Device Ip Address");
	}

	for (const FString& Component : Components)
	{
		if (!Component.IsNumeric())
		{
			return LOCTEXT("DeviceIpAddress_Numeric", "Device Ip Address must only contain numbers");
		}

		const int32 ComponentNum = FCString::Atoi(*Component);
		if (!InIpAddressRange.Contains(ComponentNum))
		{
			return LOCTEXT("DeviceIpAddress_Numeric_NumbersInRange", "Device Ip Address must only contain numbers in range [0-255]");
		}
	}

	return {};
}

FText FDeviceIpAddressCustomization::OnGetDeviceIpAddress() const
{
	TArray<void*> RawValue;
	DeviceIpAddressProperty->AccessRawData(RawValue);

	if (RawValue.Num())
	{
		const FDeviceIpAddress* DeviceAddress = static_cast<const FDeviceIpAddress*>(RawValue[0]);
		return FText::FromString(DeviceAddress->IpAddressString);
	}

	return FText::GetEmpty();
}

bool FDeviceIpAddressCustomization::OnDeviceIpAddressVerify(const FText& InText, FText& OutErrorText)
{
	FString Value = InText.ToString();

	TOptional<FText> MaybeError = VerifyIpAddress(InText.ToString(), IpAddressRange);

	if (MaybeError.IsSet())
	{
		OutErrorText = MaybeError.GetValue();
		return false;
	}

	return true;
}

void FDeviceIpAddressCustomization::OnDeviceIpAddressCommited(const FText& InText, ETextCommit::Type CommitInfo)
{
	TArray<void*> RawValue;
	DeviceIpAddressProperty->AccessRawData(RawValue);

	FString Value = InText.ToString();

	if (RawValue.Num())
	{
		GEditor->BeginTransaction(FText::Format(LOCTEXT("DeviceIpAddress_SetProperty", "Edit {0}"), DeviceIpAddressProperty->GetPropertyDisplayName()));

		DeviceIpAddressProperty->NotifyPreChange();

		FDeviceIpAddress* DeviceIpAddress = static_cast<FDeviceIpAddress*>(RawValue[0]);
		DeviceIpAddress->IpAddressString = MoveTemp(Value);

		DeviceIpAddressProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		DeviceIpAddressProperty->NotifyFinishedChangingProperties();

		GEditor->EndTransaction();
	}
}

bool FDeviceIpAddressCustomization::IsReadOnly() const
{
	return !DeviceIpAddressProperty->IsEditable();
}

#undef LOCTEXT_NAMESPACE