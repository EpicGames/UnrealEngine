// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/ToggleConnectActionCustomization.h"

#include "Customizations/DeviceIpAddressCustomization.h"

#include "DetailWidgetRow.h"
#include "SSimpleButton.h"
#include "Widgets/Images/SImage.h"

#include "LiveLinkDeviceSubsystem.h"
#include "Engine/Engine.h"

#include "LiveLinkFaceDevice.h"

#include "LiveLinkDeviceCapability_Connection.h"

#define LOCTEXT_NAMESPACE "ToggleConnectActionCustomization"

void FToggleConnectActionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle,
														FDetailWidgetRow& InHeaderRow,
														IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	Device = GetDevice();

	if (ensure(Device))
	{
		UConnectionDelegate* ConnectionDelegate = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionDelegate(Device);
		ConnectionDelegate->ConnectionChanged.AddSP(this, &FToggleConnectActionCustomization::OnConnectionStatusChanged);
	}

	PropertyHandle->MarkResetToDefaultCustomized();
	PropertyHandle->MarkHiddenByCustomization();

	InHeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget(LOCTEXT("ConnectToggleActionName", "Connect/Disconnect"))
		]
		.ValueContent()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(this, &FToggleConnectActionCustomization::GetButtonName)
			.OnClicked(this, &FToggleConnectActionCustomization::OnConnectButtonToggled)
			.IsEnabled(this, &FToggleConnectActionCustomization::IsConnectButtonEnabled)
		]
		.ResetToDefaultContent()
		[
			SNew(SButton)
			.IsFocusable(false)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding(0.0f)
			.Visibility(this, &FToggleConnectActionCustomization::GetStopActionVisibility)
			.OnClicked(this, &FToggleConnectActionCustomization::OnStopActionClicked)
			.ToolTipText(this, &FToggleConnectActionCustomization::GetStopActionTooltip)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("GenericStop"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
}

void FToggleConnectActionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle,
														  IDetailChildrenBuilder& InChildBuilder,
														  IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FToggleConnectActionCustomization::OnConnectionStatusChanged(ELiveLinkDeviceConnectionStatus InStatus)
{
	if (InStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		bIsDisconnecting = false;
	}
}

FText FToggleConnectActionCustomization::GetButtonName() const
{
	ELiveLinkDeviceConnectionStatus Status = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionStatus(Device);

	if (Status == ELiveLinkDeviceConnectionStatus::Connecting)
	{
		return LOCTEXT("ConnectingState", "Connecting");
	}
	else if (bIsDisconnecting || Status == ELiveLinkDeviceConnectionStatus::Disconnecting)
	{
		return LOCTEXT("DisconnectingState", "Disconnecting");
	}
	else if (Status == ELiveLinkDeviceConnectionStatus::Connected)
	{
		return LOCTEXT("ConnectedState", "Disconnect");
	}
	else if (Status == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		return LOCTEXT("DisconnectedState", "Connect");
	}
	else
	{
		return FText();
	}
}

FReply FToggleConnectActionCustomization::OnConnectButtonToggled()
{
	ELiveLinkDeviceConnectionStatus Status = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionStatus(Device);
	if (Status == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		ILiveLinkDeviceCapability_Connection::Execute_Connect(Device);
	}
	else if (Status == ELiveLinkDeviceConnectionStatus::Connected)
	{
		bIsDisconnecting = true;
		ILiveLinkDeviceCapability_Connection::Execute_Disconnect(Device);
	}
	else
	{
		return FReply::Unhandled();
	}
	
	return FReply::Handled();
}

bool FToggleConnectActionCustomization::IsConnectButtonEnabled() const
{
	ELiveLinkDeviceConnectionStatus Status = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionStatus(Device);

	if (Status == ELiveLinkDeviceConnectionStatus::Connecting ||
		bIsDisconnecting || Status == ELiveLinkDeviceConnectionStatus::Disconnecting)
	{
		return false;
	}

	if (CastChecked<ULiveLinkFaceDevice>(Device)->GetSettings()->IpAddress.IpAddressString.IsEmpty())
	{
		return false;
	}

	return true;
}

EVisibility FToggleConnectActionCustomization::GetStopActionVisibility() const
{
	ELiveLinkDeviceConnectionStatus Status = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionStatus(Device);

	if (Status == ELiveLinkDeviceConnectionStatus::Connecting)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

FReply FToggleConnectActionCustomization::OnStopActionClicked()
{
	ELiveLinkDeviceConnectionStatus Status = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionStatus(Device);

	if (Status == ELiveLinkDeviceConnectionStatus::Connecting)
	{
		bIsDisconnecting = true;

		ILiveLinkDeviceCapability_Connection::Execute_Disconnect(Device);

		bIsDisconnecting = false;

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FText FToggleConnectActionCustomization::GetStopActionTooltip() const
{
	return LOCTEXT("StopConnectActionTooltip", "Stop the connect procedure");
}

TObjectPtr<ULiveLinkDevice> FToggleConnectActionCustomization::GetDevice() const
{
	TArray<void*> RawValue;
	PropertyHandle->AccessRawData(RawValue);

	if (RawValue.IsEmpty())
	{
		return nullptr;
	}

	const FToggleConnectAction* ConnectAction = static_cast<const FToggleConnectAction*>(RawValue[0]);

	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	const TObjectPtr<ULiveLinkDevice>* MaybeDevice = Subsystem->GetDeviceMap().Find(ConnectAction->DeviceGuid);

	if (!MaybeDevice)
	{
		return nullptr;
	}

	if (!(*MaybeDevice)->Implements<ULiveLinkDeviceCapability_Connection>())
	{
		return nullptr;
	}

	return *MaybeDevice;
}

#undef LOCTEXT_NAMESPACE