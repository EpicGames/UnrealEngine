// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFaceDevice.h"

#include "Control/Messages/Constants.h"
#include "Control/Messages/ControlUpdate.h"

#include "HAL/FileManager.h"

#include "LiveLinkHub/ILiveLinkRecordingSession.h"

#include "CPSDataStream.h"
#include "CPSFileStream.h"

#include "LiveLinkFaceMetadata.h"
#include "LiveLinkFaceMetadataExtractor.h"
#include "Settings/CaptureManagerSettings.h"

#include "RtspMediaSource.h"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkFaceDevice, Log, All);

const ULiveLinkFaceDeviceSettings* ULiveLinkFaceDevice::GetSettings() const
{
	return GetDeviceSettings<ULiveLinkFaceDeviceSettings>();
}

TSubclassOf<ULiveLinkDeviceSettings> ULiveLinkFaceDevice::GetSettingsClass() const
{
	return ULiveLinkFaceDeviceSettings::StaticClass();
}

EDeviceHealth ULiveLinkFaceDevice::GetDeviceHealth() const
{
	const ULiveLinkFaceDeviceSettings* DeviceSettings = GetSettings();
	if (DeviceSettings->IpAddress.IpAddressString.IsEmpty())
	{
		return EDeviceHealth::Info;
	}

	if (!Device || !Device->IsConnected())
	{
		return EDeviceHealth::Warning;
	}

	return EDeviceHealth::Good;
}

FText ULiveLinkFaceDevice::GetHealthText() const
{
	const ULiveLinkFaceDeviceSettings* DeviceSettings = GetSettings();
	if (DeviceSettings->IpAddress.IpAddressString.IsEmpty())
	{
		return NSLOCTEXT("LiveLinkFaceDevice", "HealthText_NoAddress", "IP address not set");
	}

	if (!Device || !Device->IsConnected())
	{
		return NSLOCTEXT("LiveLinkFaceDevice", "HealthText_NotConnected", "Not connected");
	}

	return FText::GetEmpty();
}

void ULiveLinkFaceDevice::OnDeviceAdded()
{
	GetDeviceSettings<ULiveLinkFaceDeviceSettings>()->ConnectAction.DeviceGuid = GetDeviceId();

	Super::OnDeviceAdded();
}

void ULiveLinkFaceDevice::OnDeviceRemoved()
{
	if (Device)
	{
		Device->CancelAllExports();
	}

	ILiveLinkDeviceCapability_Connection::Execute_Disconnect(this);

	Super::OnDeviceRemoved();
}

void ULiveLinkFaceDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkDeviceSettings, DisplayName))
	{
		SetMediaSourceName(GetDisplayName().ToString());
	}

	Super::OnSettingChanged(InPropertyChangedEvent);
}

TMap<TSubclassOf<ULiveLinkDeviceCapability>, TObjectPtr<UObject>> ULiveLinkFaceDevice::OnSaveDeviceCapabilityData() const
{
	TMap<TSubclassOf<ULiveLinkDeviceCapability>, TObjectPtr<UObject>> CapabilityData;

	if (TObjectPtr<URtspMediaSource> MediaSource = Cast<URtspMediaSource>(GetMediaSource_Implementation()))
	{
		CapabilityData.Add(ULiveLinkDeviceCapability_Streaming::StaticClass(), MediaSource);
	}
	
	return CapabilityData;
}

void ULiveLinkFaceDevice::OnLoadDeviceCapabilityData(TMap<TSubclassOf<ULiveLinkDeviceCapability>, TObjectPtr<UObject>> InCapabilityDataMap)
{
	for (const TPair<TSubclassOf<ULiveLinkDeviceCapability>, TObjectPtr<UObject>>& CapabilityData : InCapabilityDataMap)
	{
		if (CapabilityData.Key == ULiveLinkDeviceCapability_Streaming::StaticClass())
		{
			if (TObjectPtr<URtspMediaSource> MediaSource = Cast<URtspMediaSource>(CapabilityData.Value))
			{
				SetMediaSource(MediaSource);
				SetMediaSourceName(GetDisplayName().ToString());
				StartStreaming_Implementation();
			}
		}
	}
}

FString ULiveLinkFaceDevice::GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const
{
	FScopeLock Lock(&DownloadedTakesMutex);
	if (const FString* DownloadedTake = DownloadedTakes.Find(InTakeId))
	{
		return *DownloadedTake;
	}
	
	return FString();
}

void ULiveLinkFaceDevice::RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback)
{
	using namespace UE::CaptureManager;

	RemoveAllTakes();

	if (!Device)
	{
		return;
	}

	TProtocolResult<TArray<FGetTakeMetadataResponse::FTakeObject>> TakesResult = Device->FetchTakeList();

	if (TakesResult.HasError())
	{
		return;
	}

	TMap<FString, FTakeId> NameToIdMap;

	for (const FGetTakeMetadataResponse::FTakeObject& Take : TakesResult.StealValue())
	{
		FTakeId TakeId = AddTake(ParseTakeMetadata(Take));
		Device->AddTakeMetadata(TakeId, Take);
		NameToIdMap.Add(Take.Name, TakeId);
	}

	FetchPreIngestFiles(MoveTemp(NameToIdMap));

	ExecuteUpdateTakeListCallback(InCallback, Execute_GetTakeIdentifiers(this));
}

void ULiveLinkFaceDevice::RunDownloadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
	using namespace UE::CaptureManager;

	FTakeId TakeId = InProcessHandle->GetTakeId();

	const FGetTakeMetadataResponse::FTakeObject& Take = Device->GetTake(TakeId);

	uint64 TotalSize = 0;
	for (const FGetTakeMetadataResponse::FFileObject& File : Take.Files)
	{
		TotalSize += File.Length;
	}

	FString DownloadedStorage = InIngestOptions->DownloadDirectory;

	TStrongObjectPtr<const UIngestCapability_Options> IngestOptions(InIngestOptions);
	TStrongObjectPtr<const UIngestCapability_ProcessHandle> ProcessHandle(InProcessHandle);

	TUniquePtr<FCPSFileStream> CPSStream = MakeUnique<FCPSFileStream>(MoveTemp(DownloadedStorage), TotalSize);
	CPSStream->SetExportFinished(FCPSFileStream::FExportFinished::CreateUObject(this, &ULiveLinkFaceDevice::OnExportFinished, Take.Name, ProcessHandle, IngestOptions));
	CPSStream->SetProgressHandler(FCPSFileStream::FReportProgress::CreateUObject(this, &ULiveLinkFaceDevice::OnExportProgressReport, ProcessHandle));

	Device->StartExport(TakeId, MoveTemp(CPSStream));
}

void ULiveLinkFaceDevice::RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
	using namespace UE::CaptureManager;

	static constexpr uint32 NumberOfTasks = 2; // Convert, Upload

	TStrongObjectPtr<const UIngestCapability_ProcessHandle> ProcessHandle(InProcessHandle);
	TStrongObjectPtr<const UIngestCapability_Options> IngestOptions(InIngestOptions);

	TSharedPtr<FTaskProgress> TaskProgress = MakeShared<FTaskProgress>
		(NumberOfTasks, FTaskProgress::FProgressReporter::CreateLambda([WeakThis = TWeakObjectPtr<ULiveLinkFaceDevice>(this), ProcessHandle](double InProgress)
			{
				TStrongObjectPtr<ULiveLinkFaceDevice> This = WeakThis.Pin();

				if (!This.IsValid())
				{
					// Don't log this one, just because it's called so frequently
					return;
				}

				This->ExecuteProcessProgressReporter(ProcessHandle.Get(), InProgress);
			}));

	// Free the current thread that is waiting on next download
	auto Task = [WeakThis = TWeakObjectPtr<ULiveLinkFaceDevice>(this), ProcessHandle, IngestOptions, TaskProgress = MoveTemp(TaskProgress)]()
		{
			TStrongObjectPtr<ULiveLinkFaceDevice> This = WeakThis.Pin();

			if (!This.IsValid())
			{
				UE_LOGF(LogLiveLinkFaceDevice, Warning, "Failed to ingest take, the device has been destroyed");
				return;
			}

			This->Super::IngestTake(ProcessHandle.Get(), IngestOptions.Get(), TaskProgress);
			This->RemoveDownloadedTakeData(ProcessHandle->GetTakeId());
		};

	UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Task), UE::Tasks::ETaskPriority::BackgroundNormal);
}

void ULiveLinkFaceDevice::CancelIngestProcess_Implementation(const UIngestCapability_ProcessHandle* InProcessHandle)
{
	if (!Device)
	{
		return;
	}

	UE::CaptureManager::FTakeId TakeId = InProcessHandle->GetTakeId();

	Device->CancelExport(TakeId);

	Super::CancelIngest(TakeId);
}

ELiveLinkDeviceConnectionStatus ULiveLinkFaceDevice::GetConnectionStatus_Implementation() const
{
	if (!Device)
	{
		return ELiveLinkDeviceConnectionStatus::Disconnected;
	}

	if (bIsConnecting)
	{
		return ELiveLinkDeviceConnectionStatus::Connecting;
	}
	else if (Device->IsConnected())
	{
		return ELiveLinkDeviceConnectionStatus::Connected;
	}

	return ELiveLinkDeviceConnectionStatus::Disconnected;
}

FString ULiveLinkFaceDevice::GetHardwareId_Implementation() const
{
	return GetSettings()->IpAddress.IpAddressString;
}

bool ULiveLinkFaceDevice::SetHardwareId_Implementation(const FString& HardwareID)
{
	return false;
}

bool ULiveLinkFaceDevice::Connect_Implementation()
{
	const ULiveLinkFaceDeviceSettings* DeviceSettings = GetSettings();

	using namespace UE::CaptureManager;
	
	if (DeviceSettings->IpAddress.IpAddressString.IsEmpty())
	{
		return false;
	}

	Device = FCPSDevice::MakeCPSDevice(DeviceSettings->IpAddress.IpAddressString, static_cast<uint16>(DeviceSettings->Port));
	
	Device->SubscribeToEvent(FConnectionStateChangedEvent::Name, FCaptureEventHandler(
		FCaptureEventHandler::Type::CreateUObject(this, &ULiveLinkFaceDevice::HandleConnectionChanged),
		EDelegateExecutionThread::AnyThread));

	Device->SubscribeToEvent(FCPSStateEvent::Name, FCaptureEventHandler(
		FCaptureEventHandler::Type::CreateUObject(this, &ULiveLinkFaceDevice::HandleCPSStateUpdate),
		EDelegateExecutionThread::AnyThread));

	Device->SubscribeToEvent(FCPSEvent::Name, FCaptureEventHandler(
		FCaptureEventHandler::Type::CreateUObject(this, &ULiveLinkFaceDevice::HandleCPSEvent),
		EDelegateExecutionThread::AnyThread));

	Device->InitiateConnect();

	return true;
}

bool ULiveLinkFaceDevice::Disconnect_Implementation()
{
	auto Task = [WeakThis = TWeakObjectPtr<ULiveLinkFaceDevice>(this)]()
		{
			TStrongObjectPtr<ULiveLinkFaceDevice> This = WeakThis.Pin();

			if (!This.IsValid())
			{
				UE_LOGF(LogLiveLinkFaceDevice, Warning, "Failed to disconnect, the device has been destroyed");
				return;
			}

			if (This->Device)
			{
				This->Device->Stop();
				This->Device->UnsubscribeAll();
				This->bIsConnecting = false;
				This->Device = nullptr;
			}
		};

	UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Task), UE::Tasks::ETaskPriority::BackgroundNormal);

	return true;
}

bool ULiveLinkFaceDevice::StartRecording_Implementation()
{
	if (!Device)
	{
		return false;
	}

	using namespace UE::CaptureManager;

	ILiveLinkRecordingSession& SessionInfo = ILiveLinkRecordingSession::Get();
	if (SessionInfo.GetSlateName().IsEmpty() || SessionInfo.GetTakeNumber() == -1)
	{
		return false;
	}

	uint16 TakeNumber = static_cast<uint16>(SessionInfo.GetTakeNumber());
	TProtocolResult<void> Result = Device->StartRecording(SessionInfo.GetSlateName(), TakeNumber);

	return Result.HasValue();
}

bool ULiveLinkFaceDevice::StopRecording_Implementation()
{
	if (!Device)
	{
		return false;
	}

	using namespace UE::CaptureManager;

	TProtocolResult<void> Result = Device->StopRecording();

	return Result.HasValue();
}

bool ULiveLinkFaceDevice::IsRecording_Implementation() const
{
	return bIsRecording;
}

void ULiveLinkFaceDevice::HandleConnectionChanged(TSharedPtr<const UE::CaptureManager::FCaptureEvent> InEvent)
{
	using namespace UE::CaptureManager;

	TSharedPtr<const FConnectionStateChangedEvent> Event = StaticCastSharedPtr<const FConnectionStateChangedEvent>(InEvent);

	if (Event->ConnectionState == FConnectionStateChangedEvent::EState::Connecting)
	{
		bIsConnecting = true;
	}
	else if (Event->ConnectionState == FConnectionStateChangedEvent::EState::Connected)
	{
		// Don't register if it is already registered
		if (!GetMediaSource_Implementation())
		{
			TObjectPtr<URtspMediaSource> RtspMediaSource = NewObject<URtspMediaSource>();
			RtspMediaSource->Host = GetSettings()->IpAddress.IpAddressString;
			RtspMediaSource->DecoderBufferSize = 1; // Using 1 for minimum latency
			RtspMediaSource->bProvideCpuBuffer = true;

			ConfigureMediaSource(RtspMediaSource);

			SetMediaSource(RtspMediaSource);
			SetMediaSourceName(GetDisplayName().ToString());
		}

		// Start streaming as soon as device is connected
		StartStreaming_Implementation();

		bIsConnecting = false;
	}
	else if (Event->ConnectionState == FConnectionStateChangedEvent::EState::Disconnected)
	{
		StopStreaming_Implementation();
	}

	auto GetConnectionState = [](FConnectionStateChangedEvent::EState InConnectionState) -> ELiveLinkDeviceConnectionStatus
	{
		switch (InConnectionState)
		{
			case FConnectionStateChangedEvent::EState::Connecting:
				return ELiveLinkDeviceConnectionStatus::Connecting;
			case FConnectionStateChangedEvent::EState::Connected:
				return ELiveLinkDeviceConnectionStatus::Connected;
			case FConnectionStateChangedEvent::EState::Disconnected:
			case FConnectionStateChangedEvent::EState::Unknown:
			default:
				return ELiveLinkDeviceConnectionStatus::Disconnected;
		}
	};

	ELiveLinkDeviceConnectionStatus ConnectionStatus = GetConnectionState(Event->ConnectionState);

	SetConnectionStatus(ConnectionStatus);
}

void ULiveLinkFaceDevice::HandleCPSStateUpdate(TSharedPtr<const UE::CaptureManager::FCaptureEvent> InEvent)
{
	using namespace UE::CaptureManager;

	TSharedPtr<const FCPSStateEvent> Event = StaticCastSharedPtr<const FCPSStateEvent>(InEvent);

	bIsRecording = Event->GetStateResponse.IsRecording();
}

void ULiveLinkFaceDevice::HandleCPSEvent(TSharedPtr<const UE::CaptureManager::FCaptureEvent> InEvent)
{
	using namespace UE::CaptureManager;

	TSharedPtr<const FCPSEvent> Event = StaticCastSharedPtr<const FCPSEvent>(InEvent);

	if (Event->UpdateMessage->GetAddressPath() == CPS::AddressPaths::GRecordingStatus)
	{
		TSharedPtr<FRecordingStatusUpdate> Update = StaticCastSharedPtr<FRecordingStatusUpdate>(Event->UpdateMessage);
		bIsRecording = Update->IsRecording();
	}
	else if (Event->UpdateMessage->GetAddressPath() == CPS::AddressPaths::GTakeAdded)
	{
		TSharedPtr<const FTakeAddedUpdate> TakeAddedUpdate = StaticCastSharedPtr<const FTakeAddedUpdate>(Event->UpdateMessage);

		TProtocolResult<FGetTakeMetadataResponse::FTakeObject> FetchTakeResult = Device->FetchTake(TakeAddedUpdate->GetTakeName());

		if (FetchTakeResult.IsValid())
		{
			FGetTakeMetadataResponse::FTakeObject Take = FetchTakeResult.StealValue();
			
			FString TakeName = Take.Name;

			FTakeId TakeId = AddTake(ParseTakeMetadata(Take));
			Device->AddTakeMetadata(TakeId, MoveTemp(Take));

			PublishEvent<FIngestCapability_TakeAddedEvent>(TakeId);

			TMap<FString, UE::CaptureManager::FTakeId> NameToIdMap;
			NameToIdMap.Add(TakeName, TakeId);

			FetchPreIngestFiles(MoveTemp(NameToIdMap));
		}
	}
	else if (Event->UpdateMessage->GetAddressPath() == CPS::AddressPaths::GTakeRemoved)
	{
		TSharedPtr<const FTakeRemovedUpdate> TakeRemoveUpdate = StaticCastSharedPtr<const FTakeRemovedUpdate>(Event->UpdateMessage);

		FTakeId TakeId = Device->GetTakeId(TakeRemoveUpdate->GetTakeName());

		if (TakeId != INDEX_NONE)
		{
			CancelIngest(TakeId);

			RemoveTake(TakeId);
			Device->RemoveTakeMetadata(TakeId);
			
			PublishEvent<FIngestCapability_TakeRemovedEvent>(TakeId);
		}
	}
}

FTakeMetadata ULiveLinkFaceDevice::ParseTakeMetadata(const UE::CaptureManager::FGetTakeMetadataResponse::FTakeObject& InTake)
{
	FTakeMetadata TakeMetadata;

	TakeMetadata.Slate = InTake.Slate;
	TakeMetadata.TakeNumber = InTake.TakeNumber;
	FDateTime DateTime = TakeMetadata.DateTime.Get(FDateTime());
	FDateTime::ParseIso8601(*InTake.DateTime, DateTime);
	TakeMetadata.DateTime = DateTime;

	FTakeMetadata::FVideo Video;
	Video.FrameRate = InTake.Video.FrameRate;
	Video.FramesCount = static_cast<uint32>(InTake.Video.Frames);
	Video.Format = TEXT("mov");
	Video.FrameHeight = InTake.Video.Height;
	Video.FrameWidth = InTake.Video.Width;

	TakeMetadata.Video.Add(MoveTemp(Video));

	return TakeMetadata;
}

void ULiveLinkFaceDevice::OnExportProgressReport(float InProgress, TStrongObjectPtr<const UIngestCapability_ProcessHandle> InProcessHandle)
{
	ExecuteProcessProgressReporter(InProcessHandle.Get(), InProgress);
}

void ULiveLinkFaceDevice::OnExportFinished(UE::CaptureManager::TProtocolResult<void> InResult,
										   FString InTakeName,
										   TStrongObjectPtr<const UIngestCapability_ProcessHandle> InProcessHandle,
										   TStrongObjectPtr<const UIngestCapability_Options> InIngestOptions)
{
	using namespace UE::CaptureManager;

	const FString DownloadedStorage = InIngestOptions->DownloadDirectory;
	const FString DownloadedTake = FPaths::Combine(DownloadedStorage, InTakeName);

	if (InResult.HasValue())
	{
		FExtractionConfig ExtractionConfig;
		if (const UCaptureManagerSettings* CaptureManagerSettings = GetDefault<UCaptureManagerSettings>())
		{
			if (CaptureManagerSettings->bEnableThirdPartyEncoder && !CaptureManagerSettings->ThirdPartyEncoder.FilePath.IsEmpty())
			{
				ExtractionConfig.bUseFFprobe = CaptureManagerSettings->bEnableThirdPartyEncoder;
				ExtractionConfig.FFmpegPath = CaptureManagerSettings->ThirdPartyEncoder.FilePath;
			}
		}

		TValueOrError<FTakeMetadata, ELiveLinkFaceExtractionError> ExtractResult = ExtractLiveLinkFaceMetadata(DownloadedTake, ExtractionConfig);

		if (ExtractResult.HasValue())
		{
			FTakeId TakeId = InProcessHandle->GetTakeId();

			FScopeLock Lock(&DownloadedTakesMutex);
			DownloadedTakes.Add(TakeId, DownloadedTake);
			Lock.Unlock();

			UpdateTake(TakeId, ExtractResult.StealValue());

			ExecuteProcessFinishedReporter(InProcessHandle.Get(), MakeValue());
		}
		else
		{
			TValueOrError<void, FIngestCapability_Error> Result = MakeError(FIngestCapability_Error::DownloaderError, TEXT("Failed to parse the take metadata"));
			ExecuteProcessFinishedReporter(InProcessHandle.Get(), MoveTemp(Result));
			IFileManager::Get().DeleteDirectory(*DownloadedTake, false, true);
		}
	}
	else
	{
		TValueOrError<void, FIngestCapability_Error> Result = MakeError(FIngestCapability_Error::DownloaderError, InResult.GetError().GetMessage());
		ExecuteProcessFinishedReporter(InProcessHandle.Get(), MoveTemp(Result));
		IFileManager::Get().DeleteDirectory(*DownloadedTake, false, true);
	}
}


void ULiveLinkFaceDevice::FetchPreIngestFiles(TMap<FString, UE::CaptureManager::FTakeId> InNameToIdMap)
{
	using namespace UE::CaptureManager;

	FCPSDataStream::FFileExportFinished OnFileExportFinishedCallback = FCPSDataStream::FFileExportFinished::CreateLambda(
		[this, NameToIdMap = MoveTemp(InNameToIdMap)](TMap<FString, TMap<FString, TProtocolResult<FCPSDataStream::FData>>> InData)
		{
			for (const TPair<FString, TMap<FString, TProtocolResult<FCPSDataStream::FData>>>& TakePair : InData)
			{
				const FString& TakeName = TakePair.Key;
				for (const TPair<FString, TProtocolResult<FCPSDataStream::FData>>& FilePair : TakePair.Value)
				{
					const FString& FileName = FilePair.Key;
					TProtocolResult<FCPSDataStream::FData> Result = FilePair.Value;

					if (Result.HasError())
					{
						continue;
					}

					if (const FTakeId* TakeId = NameToIdMap.Find(TakeName))
					{
						TOptional<FTakeMetadata> TakeMetadataOpt = GetTakeMetadata(*TakeId);
						if (!TakeMetadataOpt.IsSet())
						{
							return;
						}

						FTakeMetadata TakeMetadata = TakeMetadataOpt.GetValue();

						FCPSDataStream::FData Data = Result.GetValue();

						if (FileName == TEXT("thumbnail.jpg"))
						{
							TakeMetadata.Thumbnail = FTakeThumbnailData(MoveTemp(Data));
						}
						else if (FileName == TEXT("video_metadata.json"))
						{

							FString DataString = FString(StringCast<UTF8CHAR>(reinterpret_cast<const char*>(Data.GetData()), Data.Num()));

							TArray<FText> ValidationFailures;
							TArray<FTakeMetadata::FVideo> ParsedVideoObject = LiveLinkMetadata::ParseOldLiveLinkVideoMetadataFromString(DataString, ValidationFailures);

							TakeMetadata.Video = ParsedVideoObject;
						}

						UpdateTake(*TakeId, MoveTemp(TakeMetadata));

						PublishEvent<FIngestCapability_TakeUpdatedEvent>(*TakeId);
					}
				}
			}
		});

	TUniquePtr<FCPSDataStream> DataStream = MakeUnique<FCPSDataStream>(MoveTemp(OnFileExportFinishedCallback));

	Device->FetchFiles(MoveTemp(DataStream), { TEXT("thumbnail.jpg"), TEXT("video_metadata.json") });
}

void ULiveLinkFaceDevice::RemoveDownloadedTakeData(const UE::CaptureManager::FTakeId InTakeId)
{
	FScopeLock Lock(&DownloadedTakesMutex);
	FString* DownloadedTake = DownloadedTakes.Find(InTakeId);

	if (!DownloadedTake)
	{
		return;
	}

	IFileManager::Get().DeleteDirectory(**DownloadedTake, false, true);
}

