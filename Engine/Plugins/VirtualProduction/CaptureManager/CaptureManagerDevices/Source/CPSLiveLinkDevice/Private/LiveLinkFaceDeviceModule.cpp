// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "PropertyEditorModule.h"

#include "LiveLinkFaceDevice.h"
#include "Customizations/ToggleConnectActionCustomization.h"
#include "Customizations/DeviceIpAddressCustomization.h"

class FCPSLiveLinkDeviceModule : public IModuleInterface
{
public:

	void StartupModule()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(FToggleConnectAction::StaticStruct()->GetFName(),
														FOnGetPropertyTypeCustomizationInstance::CreateRaw(this, &FCPSLiveLinkDeviceModule::GetToggleConnectCustomization));
		PropertyModule.RegisterCustomPropertyTypeLayout(FDeviceIpAddress::StaticStruct()->GetFName(),
														FOnGetPropertyTypeCustomizationInstance::CreateRaw(this, &FCPSLiveLinkDeviceModule::GetDeviceIpAddressCustomization));
	}

	void ShutdownModule()
	{
		if (UObjectInitialized())
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomPropertyTypeLayout(FToggleConnectAction::StaticStruct()->GetFName());
		}
	}

private:

	TSharedRef<IPropertyTypeCustomization> GetToggleConnectCustomization()
	{
		return MakeShared<FToggleConnectActionCustomization>();
	}

	TSharedRef<IPropertyTypeCustomization> GetDeviceIpAddressCustomization()
	{
		return MakeShared<FDeviceIpAddressCustomization>();
	}
};

IMPLEMENT_MODULE(FCPSLiveLinkDeviceModule, CPSLiveLinkDevice);