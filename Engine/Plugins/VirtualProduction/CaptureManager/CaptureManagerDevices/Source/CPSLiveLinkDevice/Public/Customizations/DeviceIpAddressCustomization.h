// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"

#include "DeviceIpAddressCustomization.generated.h"

USTRUCT(BlueprintType)
struct CPSLIVELINKDEVICE_API FDeviceIpAddress
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DeviceAddress, DisplayName="Ip Address")
	FString IpAddressString;
};

class CPSLIVELINKDEVICE_API FDeviceIpAddressCustomization : public IPropertyTypeCustomization
{
public:
	FDeviceIpAddressCustomization();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	static TOptional<FText> VerifyIpAddress(const FString& InIpAddress, TRange<int32> InIpAddressRange);

private:
	FText OnGetDeviceIpAddress() const;
	bool OnDeviceIpAddressVerify(const FText& InText, FText& OutErrorText);
	void OnDeviceIpAddressCommited(const FText& InText, ETextCommit::Type CommitInfo);
	bool IsReadOnly() const;

	TSharedPtr<IPropertyHandle> DeviceIpAddressProperty;
	TRange<int32> IpAddressRange;
};
