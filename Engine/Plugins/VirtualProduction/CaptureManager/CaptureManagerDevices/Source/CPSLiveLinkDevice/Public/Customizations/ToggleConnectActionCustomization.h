// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"

#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability_Connection.h"

#include "ToggleConnectActionCustomization.generated.h"

USTRUCT()
struct CPSLIVELINKDEVICE_API FToggleConnectAction
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid DeviceGuid;
};

class CPSLIVELINKDEVICE_API FToggleConnectActionCustomization : public IPropertyTypeCustomization
{
public:

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, 
								 FDetailWidgetRow& InHeaderRow, 
								 IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, 
								   IDetailChildrenBuilder& InChildBuilder, 
								   IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
private:

	void OnConnectionStatusChanged(ELiveLinkDeviceConnectionStatus InStatus);

	FText GetButtonName() const;
	FReply OnConnectButtonToggled();
	bool IsConnectButtonEnabled() const;

	EVisibility GetStopActionVisibility() const;
	FReply OnStopActionClicked();
	FText GetStopActionTooltip() const;

	TObjectPtr<ULiveLinkDevice> GetDevice() const;

	TSharedPtr<IPropertyHandle> PropertyHandle;
	TObjectPtr<ULiveLinkDevice> Device;
	std::atomic_bool bIsDisconnecting = false;
};