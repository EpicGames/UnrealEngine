// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"
#include "Streaming/LiveLinkDeviceCapability_Streaming.h"
#include "Misc/Optional.h"

#include "Async/TaskProgress.h"

#include "CPSDevice.h"
#include "Customizations/ToggleConnectActionCustomization.h"
#include "Customizations/DeviceIpAddressCustomization.h"

#include "BaseIngestLiveLinkDevice.h"

#include "LiveLinkFaceDevice.generated.h"

class URtspMediaSource;


UCLASS(BlueprintType)
class CPSLIVELINKDEVICE_API ULiveLinkFaceDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:

	ULiveLinkFaceDeviceSettings()
	{
		DisplayName = TEXT("Live Link Face");
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Face")
	FDeviceIpAddress IpAddress;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Face")
	int32 Port = 14785;

	UPROPERTY(VisibleAnywhere, Category = "Live Link Face")
	FToggleConnectAction ConnectAction;
};


UCLASS(BlueprintType, meta = (DisplayName = "Live Link Face Device", ToolTip = "Use for ingest from the Live Link Face app and control recording"))
class CPSLIVELINKDEVICE_API ULiveLinkFaceDevice : public UBaseIngestLiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
	, public ILiveLinkDeviceCapability_Recording
	, public ILiveLinkDeviceCapability_Streaming
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "LiveLinkFaceDeviceSettings")
	const ULiveLinkFaceDeviceSettings* GetSettings() const;

	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;

	virtual void OnDeviceAdded() override;
	virtual void OnDeviceRemoved() override;
	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;

	virtual TMap<TSubclassOf<ULiveLinkDeviceCapability>, TObjectPtr<UObject>> OnSaveDeviceCapabilityData() const override;
	virtual void OnLoadDeviceCapabilityData(TMap<TSubclassOf<ULiveLinkDeviceCapability>, TObjectPtr<UObject>> InCapabilityDataMap) override;
	//~ End ULiveLinkDevice interface

private:
	virtual FString GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const override;

	//~ Begin ULiveLinkDeviceCapability_Ingest interface
	virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) override;
	virtual void RunDownloadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) override;
	virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) override;
	virtual void CancelIngestProcess_Implementation(const UIngestCapability_ProcessHandle* InProcessHandle) override;
	//~ End ULiveLinkDeviceCapability_Ingest interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;

protected:
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	/** Override to configure device-specific properties on the RTSP media source (e.g. Path). Called during source creation. */
	virtual void ConfigureMediaSource(URtspMediaSource* InMediaSource) {}

private:
	//~ Begin ILiveLinkDeviceCapability_Recording interface
	virtual bool StartRecording_Implementation() override;
	virtual bool StopRecording_Implementation() override;
	virtual bool IsRecording_Implementation() const override;
	//~ End ILiveLinkDeviceCapability_Recording interface

	void HandleConnectionChanged(TSharedPtr<const UE::CaptureManager::FCaptureEvent> InEvent);
	void HandleCPSStateUpdate(TSharedPtr<const UE::CaptureManager::FCaptureEvent> InEvent);
	void HandleCPSEvent(TSharedPtr<const UE::CaptureManager::FCaptureEvent> InEvent);

	void OnExportProgressReport(float InProgress, TStrongObjectPtr<const UIngestCapability_ProcessHandle> InProcessHandle);
	void OnExportFinished(UE::CaptureManager::TProtocolResult<void> InResult,
						  FString InTakeName,
						  TStrongObjectPtr<const UIngestCapability_ProcessHandle> InProcessHandle,
						  TStrongObjectPtr<const UIngestCapability_Options> InIngestOptions);

	FTakeMetadata ParseTakeMetadata(const UE::CaptureManager::FGetTakeMetadataResponse::FTakeObject& InTake);

	void FetchPreIngestFiles(TMap<FString, UE::CaptureManager::FTakeId> InNameToIdMap);

	void RemoveDownloadedTakeData(const UE::CaptureManager::FTakeId InTakeId);

	TSharedPtr<UE::CaptureManager::FCPSDevice> Device;

	mutable FCriticalSection DownloadedTakesMutex;
	TMap<UE::CaptureManager::FTakeId, FString> DownloadedTakes;

	std::atomic_bool bIsRecording = false;
	std::atomic_bool bIsConnecting = false;
};
