// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.8, "This header has moved to CaptureManagerCPSClient. Include \"CPSDevice.h\" from the CaptureManagerCPSClient module instead.")

#include "CPSDevice.h"
