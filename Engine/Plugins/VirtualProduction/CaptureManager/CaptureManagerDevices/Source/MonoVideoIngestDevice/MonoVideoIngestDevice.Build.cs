// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MonoVideoIngestDevice : ModuleRules
{
	public MonoVideoIngestDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "MonoVideoIngest";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Engine",
			"Json",
			"CaptureManagerMediaRW",
			"IngestLiveLinkDevice",
			"LiveLinkDevice",
			"LiveLinkCapabilities",
			"Media",
			"LiveLinkHubCaptureMessaging",
			"LiveLinkHub",
			"ToolWidgets",
			"Slate",
			"SlateCore",
			"CaptureManagerTakeMetadata",
			"CaptureManagerSettings",
			"CoreUObject",
			"UnrealEd",
			"VideoLiveLinkDeviceCommon",
			"CaptureMetadataExtraction"
		});

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
