// Copyright Epic Games, Inc. All Rights Reserved.

#include "MonoVideoIngestDevice.h"

#include "HAL/FileManager.h"
#include "MonoVideoMetadataExtractor.h"
#include "Settings/CaptureManagerSettings.h"

#include "Utils/TakeDiscoveryExpressionParser.h"
#include "Utils/VideoDeviceThumbnailExtractor.h"

DEFINE_LOG_CATEGORY_STATIC(LogMonoVideoIngestDevice, Log, All);

#define LOCTEXT_NAMESPACE "MonoVideoDevice"

namespace UE::MonoVideoLiveLinkDevice::Private
{

static const TArray<FString::ElementType> Delimiters =
{
	'-',
	'_',
	'.'
};

static const TArray<FStringView> SupportedVideoFormats =
{
	TEXTVIEW("mp4"),
	TEXTVIEW("mov")
};


}

const UMonoVideoIngestDeviceSettings* UMonoVideoIngestDevice::GetSettings() const
{
	return GetDeviceSettings<UMonoVideoIngestDeviceSettings>();
}

TSubclassOf<ULiveLinkDeviceSettings> UMonoVideoIngestDevice::GetSettingsClass() const
{
	return UMonoVideoIngestDeviceSettings::StaticClass();
}

EDeviceHealth UMonoVideoIngestDevice::GetDeviceHealth() const
{
	return EDeviceHealth::Nominal;
}

FText UMonoVideoIngestDevice::GetHealthText() const
{
	return FText::FromString("Nominal");
}

void UMonoVideoIngestDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::OnSettingChanged(InPropertyChangedEvent);

	const FName& PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (GET_MEMBER_NAME_CHECKED(UMonoVideoIngestDeviceSettings, TakeDirectory) == PropertyName
		|| GET_MEMBER_NAME_CHECKED(UMonoVideoIngestDeviceSettings, VideoDiscoveryExpression) == PropertyName)
	{
		SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);

		FString Path = GetSettings()->TakeDirectory.Path;
		if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
		{
			SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
		}
		else
		{
			SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);
		}
	}
}

FString UMonoVideoIngestDevice::GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const
{
	if (const FString* TakePath = FullTakePaths.Find(InTakeId))
	{
		return *TakePath;
	}

	return FString();
}

void UMonoVideoIngestDevice::RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
	auto Task = [WeakThis = TWeakObjectPtr<UMonoVideoIngestDevice>(this),
		ProcessHandle = TStrongObjectPtr<const UIngestCapability_ProcessHandle>(InProcessHandle),
		IngestOptions = TStrongObjectPtr<const UIngestCapability_Options>(InIngestOptions)]()
		{
			TStrongObjectPtr<UMonoVideoIngestDevice> This = WeakThis.Pin();

			if (!This.IsValid())
			{
				UE_LOGF(LogMonoVideoIngestDevice, Warning, "Failed to ingest take, the device has been destroyed");
				return;
			}

			static constexpr uint32 NumberOfTasks = 2; // Convert, Upload

			using namespace UE::CaptureManager;
			TSharedPtr<FTaskProgress> TaskProgress
				= MakeShared<FTaskProgress>(NumberOfTasks, FTaskProgress::FProgressReporter::CreateLambda([WeakThis = WeakThis, ProcessHandle](double InProgress)
					{
						TStrongObjectPtr<UMonoVideoIngestDevice> This = WeakThis.Pin();

						if (!This.IsValid())
						{
							// Don't log this one, just because it's called so frequently
							return;
						}

						This->ExecuteProcessProgressReporter(ProcessHandle.Get(), InProgress);
					}));

			This->Super::IngestTake(ProcessHandle.Get(), IngestOptions.Get(), MoveTemp(TaskProgress));
		};

	UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Task), UE::Tasks::ETaskPriority::BackgroundNormal);
}

void UMonoVideoIngestDevice::RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback)
{
	RemoveAllTakes();

	FString StoragePath = GetSettings()->TakeDirectory.Path;
	int32 DirectoriesInterrogatedCount = 0;
	const int32 DirectoriesToInterrogateInOneRun = 200;

	TArray<FString> SupportedVideoFiles;
	const bool bIterationResult = IFileManager::Get().IterateDirectoryRecursively(*StoragePath, [&](const TCHAR* InFileNameOrDirectory, bool bInIsDirectory)
		{
			if (IsUpdateTakeListAbortRequested())
			{
				return false;
			}

			if (bInIsDirectory)
			{
				if (++DirectoriesInterrogatedCount > DirectoriesToInterrogateInOneRun)
				{
					return false;
				}
			}

			if (!bInIsDirectory && IsVideoFile(InFileNameOrDirectory))
			{
				SupportedVideoFiles.Add(InFileNameOrDirectory);
			}

			return true;
		});

	for (const FString& SupportedVideoFile : SupportedVideoFiles)
	{
		if (IsUpdateTakeListAbortRequested())
		{
			ExecuteUpdateTakeListCallback(InCallback, TArray<int32>());
			return;
		}

		TOptional<FTakeMetadata> TakeInfoResult = ReadTake(SupportedVideoFile);

		if (TakeInfoResult.IsSet())
		{
			int32 TakeId = AddTake(TakeInfoResult.GetValue());
			FullTakePaths.Add(TakeId, SupportedVideoFile);

			PublishEvent<FIngestCapability_TakeAddedEvent>(TakeId);
		}
	}

	ExecuteUpdateTakeListCallback(InCallback, Execute_GetTakeIdentifiers(this));
}

ELiveLinkDeviceConnectionStatus UMonoVideoIngestDevice::GetConnectionStatus_Implementation() const
{
	const UMonoVideoIngestDeviceSettings* DeviceSettings = GetSettings();
	FString Path = DeviceSettings->TakeDirectory.Path;

	if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
	{
		return ELiveLinkDeviceConnectionStatus::Connected;
	}

	return ELiveLinkDeviceConnectionStatus::Disconnected;
}

FString UMonoVideoIngestDevice::GetHardwareId_Implementation() const
{
	return FPlatformMisc::GetDeviceId();
}

bool UMonoVideoIngestDevice::SetHardwareId_Implementation(const FString& HardwareID)
{
	return false;
}

bool UMonoVideoIngestDevice::Connect_Implementation()
{
	const UMonoVideoIngestDeviceSettings* DeviceSettings = GetSettings();
	FString Path = DeviceSettings->TakeDirectory.Path;

	if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
	{
		SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
		return true;
	}

	return false;
}

bool UMonoVideoIngestDevice::Disconnect_Implementation()
{
	SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);
	return true;
}

bool UMonoVideoIngestDevice::IsVideoFile(const FString& InFileNameWithExtension)
{
	FString Extension = FPaths::GetExtension(InFileNameWithExtension);

	return UE::MonoVideoLiveLinkDevice::Private::SupportedVideoFormats.Contains(Extension);
}

TOptional<FTakeMetadata> UMonoVideoIngestDevice::ReadTake(const FString& InCurrentTakeFile) const
{
	using namespace UE::CaptureManager;

	FString FileNameFormat = GetSettings()->VideoDiscoveryExpression.Value;
	FString FileName = FPaths::GetBaseFilename(InCurrentTakeFile);

	FString SlateName;
	FString Name;
	int32 TakeNumber = INDEX_NONE;

	if (FileNameFormat != "<Auto>")
	{
		FTakeDiscoveryExpressionParser TokenParser(FileNameFormat, FileName, UE::MonoVideoLiveLinkDevice::Private::Delimiters);
		if (!TokenParser.Parse())
		{
			UE_LOGF(LogMonoVideoIngestDevice, Warning, "Failed to match the specified format (%ls) with the video file (%ls)", *(FileNameFormat), *InCurrentTakeFile);
			return {};
		}

		SlateName = TokenParser.GetSlateName();
		TakeNumber = TokenParser.GetTakeNumber();
		Name = TokenParser.GetName();
	}

	if (TakeNumber == INDEX_NONE)
	{
		TakeNumber = 1;
	}

	if (Name.IsEmpty())
	{
		Name = TEXT("video");
	}

	FExtractionConfig ExtractionConfig;
	if (const UCaptureManagerSettings* CaptureManagerSettings = GetDefault<UCaptureManagerSettings>())
	{
		if (CaptureManagerSettings->bEnableThirdPartyEncoder && !CaptureManagerSettings->ThirdPartyEncoder.FilePath.IsEmpty())
		{
			ExtractionConfig.bUseFFprobe = CaptureManagerSettings->bEnableThirdPartyEncoder;
			ExtractionConfig.FFmpegPath = CaptureManagerSettings->ThirdPartyEncoder.FilePath;
		}
	}

	FMonoVideoDescriptor Descriptor;
	Descriptor.VideoFilePath = InCurrentTakeFile;
	Descriptor.Slate = SlateName;
	Descriptor.TakeNumber = TakeNumber;

	TValueOrError<FTakeMetadata, EMonoVideoExtractionError> Result = ExtractMonoVideoMetadata(MoveTemp(Descriptor), ExtractionConfig);

	if (Result.HasError())
	{
		UE_LOGF(LogMonoVideoIngestDevice, Error, "Failed to extract metadata for video file \"%ls\"", *FPaths::GetCleanFilename(InCurrentTakeFile));
		return {};
	}

	FTakeMetadata TakeMetadata = Result.StealValue();

	// The Core extractor derives Video.Name from the file path. Override with the
	// name resolved from the device's discovery expression parser.
	if (TakeMetadata.Video.Num() > 0)
	{
		TakeMetadata.Video[0].Name = MoveTemp(Name);

		if (!TakeMetadata.Video[0].TimecodeStart.IsSet())
		{
			UE_LOGF(LogMonoVideoIngestDevice, Warning, "Failed to determine the timecode for the video file \"%ls\".", *FPaths::GetCleanFilename(InCurrentTakeFile));
		}

		if (FMath::IsNearlyZero(TakeMetadata.Video[0].FrameRate))
		{
			UE_LOGF(LogMonoVideoIngestDevice, Warning, "Failed to determine frame rate for video file: \"%ls\"", *FPaths::GetCleanFilename(InCurrentTakeFile));
		}
	}

	// Thumbnail extraction is device-specific (not part of the Core extractor)
	FVideoDeviceThumbnailExtractor ThumbnailExtractor;
	TOptional<FTakeThumbnailData::FRawImage> RawImageOpt = ThumbnailExtractor.ExtractThumbnail(InCurrentTakeFile);
	if (RawImageOpt.IsSet())
	{
		TakeMetadata.Thumbnail = MoveTemp(RawImageOpt.GetValue());
	}

	return TakeMetadata;
}

#undef LOCTEXT_NAMESPACE
