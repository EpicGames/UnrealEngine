// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "Customizations/TakeDiscoveryExpressionCustomization.h"

#include "Misc/Optional.h"

#include "Async/TaskProgress.h"

#include "BaseIngestLiveLinkDevice.h"

#include "MonoVideoIngestDevice.generated.h"


UCLASS(BlueprintType)
class UMonoVideoIngestDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:
	UMonoVideoIngestDeviceSettings()
	{
		VideoDiscoveryExpression = TEXT("<Auto>");
		DisplayName = TEXT("Mono Video Ingest");
	}

	/* Path to a directory containing the take(s) data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mono Video Ingest")
	FDirectoryPath TakeDirectory;

	/**
	 * Format of the file name that the device expects
	 *		Available expressions:
	 *			* <Slate> - slate name
	 *			* <Name> - identifier for the camera
	 *			* <Take> - take number
	 *			* <Any> - any string
	 *			* <Auto> - used alone to automatically figure out takes based on directory structure
	 *		Available delimiters: '_', '-', '.'
	 *
	 * Examples:
	 * Considering input video MyTakeFolder/MySlate_MyName_SomeString-005.mov
	 *
	 * Video File Name Pattern <Auto> - will set Slate to the video filename “MySlate_MyName_SomeString_005” (dash converted to underline), Name will be set to just "video" and Take will be set to 1 by default.
	 *
	 * Video File Name Pattern <Slate>_<Name>_<Any>-<Take> - will extract Slate as “MySlate”, Name as “MyName” and Take as 5. <Any> ensures that the “SomeString” part of the filename is ignored.
	 *
	 * In both cases, extension '.mov' is stripped out.
	 *
	 * Note: when not using <Auto> expression, then both <Slate> and <Name> must be present in the pattern.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mono Video Ingest")
	FTakeDiscoveryExpression VideoDiscoveryExpression;
};


UCLASS(BlueprintType, meta = (DisplayName = "Mono Video Ingest", ToolTip = "Ingest video files as mono takes"))
class UMonoVideoIngestDevice final : public UBaseIngestLiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "GenericArchiveDeviceSettings")
	const UMonoVideoIngestDeviceSettings* GetSettings() const;

	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;

	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

private:
	virtual FString GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const override;

	//~ Begin ULiveLinkDeviceCapability_Ingest interface
	virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) override;
	virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) override;
	//~ End ULiveLinkDeviceCapability_Ingest interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	bool IsVideoFile(const FString& InFileNameWithExtension);

	TOptional<FTakeMetadata> ReadTake(const FString& InCurrentVideoFile) const;

	TMap<int32, FString> FullTakePaths;
};
