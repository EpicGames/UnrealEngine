// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "PropertyEditorModule.h"
#include "Customizations/TakeDiscoveryExpressionCustomization.h"
#include "Templates/SharedPointer.h"

class FMonoVideoIngestDeviceModule : public IModuleInterface
{
public:

	void StartupModule()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(FTakeDiscoveryExpression::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateRaw(this, &FMonoVideoIngestDeviceModule::GetTakeDiscoveryExpressionCustomization));
	}

	void ShutdownModule()
	{
		if (UObjectInitialized())
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomPropertyTypeLayout(FTakeDiscoveryExpression::StaticStruct()->GetFName());
		}
	}

private:

	TSharedRef<IPropertyTypeCustomization> GetTakeDiscoveryExpressionCustomization()
	{
		return MakeShared<FTakeDiscoveryExpressionCustomization>();
	}
};

IMPLEMENT_MODULE(FMonoVideoIngestDeviceModule, MonoVideoIngestDevice)