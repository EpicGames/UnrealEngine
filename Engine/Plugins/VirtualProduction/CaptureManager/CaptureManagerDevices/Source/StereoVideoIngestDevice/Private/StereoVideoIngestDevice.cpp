// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoVideoIngestDevice.h"

#include "HAL/FileManager.h"
#include "ImageUtils.h"
#include "Misc/FileHelper.h"
#include "Settings/CaptureManagerSettings.h"
#include "StereoVideoMetadataExtractor.h"

#include "Asset/CaptureAssetSanitization.h"
#include "Utils/TakeDiscoveryExpressionParser.h"
#include "Utils/VideoDeviceThumbnailExtractor.h"

DEFINE_LOG_CATEGORY_STATIC(LogStereoVideoIngestDevice, Log, All);

#define LOCTEXT_NAMESPACE "StereoVideoLLDevice"

namespace UE::StereoVideoLiveLinkDevice::Private
{
	static const TArray<FString::ElementType> Delimiters =
	{
		'-',
		'_',
		'.',
		'/'
	};

	static const TArray<FStringView> SupportedVideoExtensions =
	{
		TEXTVIEW("mp4"),
		TEXTVIEW("mov")
	};

	static const TArray<FStringView> SupportedAudioExtensions =
	{
		TEXTVIEW("wav")
	};

	static const TArray<FStringView> SupportedImageSequenceExtensions =
	{
		TEXTVIEW("jpg"),
		TEXTVIEW("jpeg"),
		TEXTVIEW("png")
	};

	static constexpr int32 MaximumVideoFilesCountPerTake = 2;
	static constexpr int32 MaximumAudioFilesCountPerTake = 1;

	static constexpr int32 DirectoriesCountToIterateForTakesSearch = 2000;
	static constexpr int32 DirectoriesCountToIterateForVideoAudioFilesSearch = 500;
}

const UStereoVideoIngestDeviceSettings* UStereoVideoIngestDevice::GetSettings() const
{
	return GetDeviceSettings<UStereoVideoIngestDeviceSettings>();
}

TSubclassOf<ULiveLinkDeviceSettings> UStereoVideoIngestDevice::GetSettingsClass() const
{
	return UStereoVideoIngestDeviceSettings::StaticClass();
}

EDeviceHealth UStereoVideoIngestDevice::GetDeviceHealth() const
{
	return EDeviceHealth::Nominal;
}

FText UStereoVideoIngestDevice::GetHealthText() const
{
	return FText::FromString("Nominal");
}

void UStereoVideoIngestDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::OnSettingChanged(InPropertyChangedEvent);

	const FName& PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (GET_MEMBER_NAME_CHECKED(UStereoVideoIngestDeviceSettings, TakeDirectory) == PropertyName
		|| GET_MEMBER_NAME_CHECKED(UStereoVideoIngestDeviceSettings, VideoDiscoveryExpression) == PropertyName
		|| GET_MEMBER_NAME_CHECKED(UStereoVideoIngestDeviceSettings, AudioDiscoveryExpression) == PropertyName)
	{
		SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);

		FString Path = GetSettings()->TakeDirectory.Path;
		if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
		{
			SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
		}
		else
		{
			SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);
		}
	}
}

FString UStereoVideoIngestDevice::GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const
{
	if (const FString* TakePath = FullTakePaths.Find(InTakeId))
	{
		return *TakePath;
	}

	return FString();
}

void UStereoVideoIngestDevice::RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
	auto Task = [WeakThis = TWeakObjectPtr<UStereoVideoIngestDevice>(this),
		ProcessHandle = TStrongObjectPtr<const UIngestCapability_ProcessHandle>(InProcessHandle),
		IngestOptions = TStrongObjectPtr<const UIngestCapability_Options>(InIngestOptions)]()
		{
			TStrongObjectPtr<UStereoVideoIngestDevice> This = WeakThis.Pin();

			if (!This.IsValid())
			{
				UE_LOGF(LogStereoVideoIngestDevice, Warning, "Failed to ingest take, the device has been destroyed");
				return;
			}

			static constexpr uint32 NumberOfTasks = 2;

			using namespace UE::CaptureManager;
			TSharedPtr<FTaskProgress> TaskProgress
				= MakeShared<FTaskProgress>(NumberOfTasks, FTaskProgress::FProgressReporter::CreateLambda([WeakThis = WeakThis, ProcessHandle](double InProgress)
					{
						TStrongObjectPtr<UStereoVideoIngestDevice> This = WeakThis.Pin();

						if (!This.IsValid())
						{
							// Don't log this one, just because it's called so frequently
							return;
						}

						This->ExecuteProcessProgressReporter(ProcessHandle.Get(), InProgress);
					}));

			This->Super::IngestTake(ProcessHandle.Get(), IngestOptions.Get(), MoveTemp(TaskProgress));
		};

	UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Task), UE::Tasks::ETaskPriority::BackgroundNormal);
}

int32 UStereoVideoIngestDevice::FTakeWithComponents::CountComponents(UStereoVideoIngestDevice::ETakeComponentType Type)
{
	int32 Count = 0;
	for (const UStereoVideoIngestDevice::FTakeWithComponents::Component& Component : Components)
	{
		if (Component.Type == Type)
		{
			++Count;
		}
	}
	return Count;
}

void UStereoVideoIngestDevice::RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback)
{
	RemoveAllTakes();

	FString StoragePath = GetSettings()->TakeDirectory.Path;

	TMap<FString, UStereoVideoIngestDevice::FTakeWithComponents> TakesWithComponentsCandidates = DiscoverTakes(StoragePath);

	TArray<FString> SlateNames;
	TakesWithComponentsCandidates.GetKeys(SlateNames);

	for (const FString& SlateName : SlateNames)
	{
		if (IsUpdateTakeListAbortRequested())
		{
			ExecuteUpdateTakeListCallback(InCallback, TArray<int32>());
			return;
		}

		FTakeWithComponents& TakeWithComponents = TakesWithComponentsCandidates[SlateName];

		bool TakeAsExpected = ((TakeWithComponents.CountComponents(ETakeComponentType::VIDEO) == UE::StereoVideoLiveLinkDevice::Private::MaximumVideoFilesCountPerTake && TakeWithComponents.CountComponents(ETakeComponentType::IMAGE_SEQUENCE) == 0) ||
							   (TakeWithComponents.CountComponents(ETakeComponentType::VIDEO) == 0 && TakeWithComponents.CountComponents(ETakeComponentType::IMAGE_SEQUENCE) == UE::StereoVideoLiveLinkDevice::Private::MaximumVideoFilesCountPerTake)) &&
			TakeWithComponents.CountComponents(ETakeComponentType::AUDIO) <= UE::StereoVideoLiveLinkDevice::Private::MaximumAudioFilesCountPerTake;

		if (!TakeAsExpected)
		{
			UE_LOGF(LogStereoVideoIngestDevice, Warning, "Invalid take at '%ls'. Take should have exactly two video files or two image sequences. Optionally, one audio file.", *TakeWithComponents.TakePath);
			continue;
		}

		TOptional<FTakeMetadata> TakeInfoResult = CreateTakeMetadata(TakeWithComponents);

		if (TakeInfoResult.IsSet())
		{
			FTakeMetadata TakeMetadata = TakeInfoResult.GetValue();

			int32 TakeId = AddTake(MoveTemp(TakeMetadata));
			FullTakePaths.Add(TakeId, TakeWithComponents.TakePath);

			PublishEvent<FIngestCapability_TakeAddedEvent>(TakeId);
		}
	}

	ExecuteUpdateTakeListCallback(InCallback, Execute_GetTakeIdentifiers(this));
}

TMap<FString, UStereoVideoIngestDevice::FTakeWithComponents> UStereoVideoIngestDevice::DiscoverTakes(FString StoragePath)
{
	TMap<FString, FTakeWithComponents> TakesWithComponentsCandidates;

	int32 DirectoriesInterrogatedCount = 0;

	const bool bIterationResult = IFileManager::Get().IterateDirectoryRecursively(*StoragePath, [&](const TCHAR* InPath, bool bInIsDirectory)
		{
			if (IsUpdateTakeListAbortRequested())
			{
				return false;
			}

			if (bInIsDirectory)
			{
				if (++DirectoriesInterrogatedCount > UE::StereoVideoLiveLinkDevice::Private::DirectoriesCountToIterateForTakesSearch)
				{
					return false;
				}
			}

			FString Extension = FPaths::GetExtension(InPath);
			if (UE::StereoVideoLiveLinkDevice::Private::SupportedVideoExtensions.Contains(Extension))
			{
				ExtractTakeComponents(InPath, StoragePath, ETakeComponentType::VIDEO, GetSettings()->VideoDiscoveryExpression.Value, "UnknownVideoName", TakesWithComponentsCandidates);
			}
			else if (UE::StereoVideoLiveLinkDevice::Private::SupportedImageSequenceExtensions.Contains(Extension))
			{
				ExtractTakeComponents(InPath, StoragePath, ETakeComponentType::IMAGE_SEQUENCE, GetSettings()->VideoDiscoveryExpression.Value, "UnknownImageSequenceName", TakesWithComponentsCandidates);
			}
			else if (UE::StereoVideoLiveLinkDevice::Private::SupportedAudioExtensions.Contains(Extension))
			{
				ExtractTakeComponents(InPath, StoragePath, ETakeComponentType::AUDIO, GetSettings()->AudioDiscoveryExpression.Value, "UnknownAudioName", TakesWithComponentsCandidates);
			}

			return true;
		});

	return TakesWithComponentsCandidates;
}

void UStereoVideoIngestDevice::ExtractTakeComponents(FString ComponentPath, FString StoragePath, ETakeComponentType ComponentType, FString Format, FString UnknownComponentName, TMap<FString, UStereoVideoIngestDevice::FTakeWithComponents>& OutTakesWithComponentsCandidates)
{
	TOptional<FTakeWithComponents> TakeWithComponents;

	if (Format == TEXT("<Auto>"))
	{
		TakeWithComponents = ExtractTakeComponentsFromDirectoryStructure(ComponentPath, StoragePath, ComponentType);
	}
	else
	{
		TakeWithComponents = ExtractTakeComponentsUsingTokens(ComponentPath, StoragePath, Format, ComponentType);
	}

	if (!TakeWithComponents.IsSet())
	{
		FString ComponentRootPath;
		FString FileName;
		FString Extension;
		FPaths::Split(ComponentPath, ComponentRootPath, FileName, Extension);

		TakeWithComponents = { ComponentRootPath, "Slate name not determined", -1, {{UnknownComponentName, ComponentType, ComponentPath}} };
	}

	GroupFoundComponents(OutTakesWithComponentsCandidates, TakeWithComponents.GetValue());
};

TOptional<UStereoVideoIngestDevice::FTakeWithComponents> UStereoVideoIngestDevice::ExtractTakeComponentsUsingTokens(FString ComponentPath, FString StoragePath, FString Format, ETakeComponentType ComponentType)
{
	FStringView InPathStringView(ComponentPath);
	FString RelativePath = FString(InPathStringView.SubStr(StoragePath.Len(), InPathStringView.Len() - StoragePath.Len()));
	FString Extension = FPaths::GetExtension(RelativePath);
	FString RelativePathNoExtension = RelativePath.Mid(0, RelativePath.Len() - Extension.Len() - 1);

	FTakeDiscoveryExpressionParser TokenParser(Format, RelativePathNoExtension, UE::StereoVideoLiveLinkDevice::Private::Delimiters);

	FString StorageLeafWithRelativePath = FPaths::GetPathLeaf(StoragePath) + RelativePathNoExtension;
	FTakeDiscoveryExpressionParser WithLeafTokenParser(Format, StorageLeafWithRelativePath, UE::StereoVideoLiveLinkDevice::Private::Delimiters);

	FString TakePath;
	FTakeDiscoveryExpressionParser* SuccessfulParser = nullptr;
	if (TokenParser.Parse())
	{
		SuccessfulParser = &TokenParser;
		TArray<FString> RelativePathParts;
		FPaths::NormalizeDirectoryName(RelativePath);
		RelativePath.ParseIntoArray(RelativePathParts, TEXT("/"));
		TakePath = StoragePath + TEXT("/") + RelativePathParts[0];
	}
	else if (WithLeafTokenParser.Parse())
	{
		SuccessfulParser = &WithLeafTokenParser;
		TakePath = StoragePath;
	}

	if (SuccessfulParser != nullptr)
	{
		return FTakeWithComponents{ TakePath, SuccessfulParser->GetSlateName(), SuccessfulParser->GetTakeNumber(), { { SuccessfulParser->GetName(), ComponentType, ComponentPath } } };
	}

	return {};
};

TOptional<UStereoVideoIngestDevice::FTakeWithComponents> UStereoVideoIngestDevice::ExtractTakeComponentsFromDirectoryStructure(FString ComponentPath, FString StoragePath, ETakeComponentType ComponentType)
{
	FStringView InPathStringView(ComponentPath);
	FString RelativePath = FString(InPathStringView.SubStr(StoragePath.Len(), InPathStringView.Len() - StoragePath.Len()));

	FString TakePath;
	FString TakeName;

	if (ComponentType == ETakeComponentType::VIDEO || ComponentType == ETakeComponentType::AUDIO)
	{
		// As a default, assume that user user navigates StoragePath to TakeFolder.
		// Then use specified storage path leaf folder as TakeName
		TakeName = FPaths::GetPathLeaf(StoragePath);
		TakePath = StoragePath;


		// Run a quick count of video and audio files in the selected StoragePath
		bool bMoreAudioOrVideoFilesThanExpectedPerTake = false;
		{
			int32 DirectoriesInterrogatedCount = 0;
			int32 VideoFilesFound = 0;
			int32 AudioFilesFound = 0;

			const bool bIterationResult = IFileManager::Get().IterateDirectoryRecursively(*StoragePath, [&](const TCHAR* InPath, bool bInIsDirectory)
				{
					if (bInIsDirectory)
					{
						if (++DirectoriesInterrogatedCount > UE::StereoVideoLiveLinkDevice::Private::DirectoriesCountToIterateForVideoAudioFilesSearch)
						{
							return false;
						}
					}

					FString Extension = FPaths::GetExtension(InPath);
					if (UE::StereoVideoLiveLinkDevice::Private::SupportedVideoExtensions.Contains(Extension))
					{
						VideoFilesFound++;
					}
					else if (UE::StereoVideoLiveLinkDevice::Private::SupportedAudioExtensions.Contains(Extension))
					{
						AudioFilesFound++;
					}

					if (VideoFilesFound > UE::StereoVideoLiveLinkDevice::Private::MaximumVideoFilesCountPerTake 
						|| AudioFilesFound > UE::StereoVideoLiveLinkDevice::Private::MaximumAudioFilesCountPerTake)
					{
						// Once we found more video or audio files than expected for one take, stop iteration over folders
						bMoreAudioOrVideoFilesThanExpectedPerTake = true;
						return false;
					}

					return true;
				});
		}

		// If specified StoragePath contains more than maximum video files count or more than maximum audio files count,
		// further assumption is that user selected a folder that is a container to take folders. 
		if (bMoreAudioOrVideoFilesThanExpectedPerTake)
		{
			// If sub folder exist, use it's name as TakeName
			TArray<FString> Parts;
			FPaths::NormalizeDirectoryName(RelativePath);
			RelativePath.ParseIntoArray(Parts, TEXT("/"));
			if (Parts.Num() > 1)
			{
				TakeName = Parts[0];
				TakePath = StoragePath / TakeName;
			}
			else
			{
				// Invalid take folder
				return {};
			}
		}
	}
	else if (ComponentType == ETakeComponentType::IMAGE_SEQUENCE)
	{
		// Assumption is that frames are stored in separate folders
		TArray<FString> Parts;
		FPaths::NormalizeDirectoryName(RelativePath);
		RelativePath.ParseIntoArray(Parts, TEXT("/"));
		if (Parts.Num() > 2)
		{
			TakeName = Parts[0];
			TakePath = StoragePath / TakeName;
		}
		else
		{
			TakePath = StoragePath;
			TakeName = FPaths::GetPathLeaf(StoragePath);
		}

		{
			FString Filename, Extension;
			FPaths::Split(ComponentPath, ComponentPath, Filename, Extension);
		}
	}

	if (TakeName.IsEmpty())
	{
		return {}; // Failed to match TakeName
	}

	FString SlateName = TakeName;
	int32 TakeNumber = 1;

	FString ComponentName = FPaths::GetCleanFilename(ComponentPath);
	UE::CaptureManager::SanitizePackagePath(ComponentName, '_');

	return FTakeWithComponents { TakePath, SlateName, TakeNumber, { { ComponentName, ComponentType, ComponentPath } } };
};

void UStereoVideoIngestDevice::GroupFoundComponents(TMap<FString, UStereoVideoIngestDevice::FTakeWithComponents>& TakesWithComponentsCandidates, FTakeWithComponents TakeWithComponents)
{
	FString TakeIdentifier = TakeWithComponents.SlateName + FString::FromInt(TakeWithComponents.TakeNumber);

	FTakeWithComponents* TakeWithComponentsPtr = TakesWithComponentsCandidates.Find(TakeIdentifier);
	if (TakeWithComponentsPtr == nullptr)
	{
		TakeWithComponentsPtr = &TakesWithComponentsCandidates.Add(TakeIdentifier, { TakeWithComponents.TakePath, TakeWithComponents.SlateName, TakeWithComponents.TakeNumber, {} });
	}

	for (FTakeWithComponents::Component& Component : TakeWithComponents.Components)
	{
		FTakeWithComponents::Component* TakeComponentPtr = TakeWithComponentsPtr->Components.FindByPredicate(
			[&Component](const FTakeWithComponents::Component& Other)
			{
				return Other.Path == Component.Path;
			});

		if (TakeComponentPtr == nullptr)
		{
			TakeWithComponentsPtr->Components.Add(MoveTemp(Component));
		}
	}
};

ELiveLinkDeviceConnectionStatus UStereoVideoIngestDevice::GetConnectionStatus_Implementation() const
{
	const UStereoVideoIngestDeviceSettings* DeviceSettings = GetSettings();
	FString Path = DeviceSettings->TakeDirectory.Path;

	if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
	{
		return ELiveLinkDeviceConnectionStatus::Connected;
	}

	return ELiveLinkDeviceConnectionStatus::Disconnected;
}

FString UStereoVideoIngestDevice::GetHardwareId_Implementation() const
{
	return FPlatformMisc::GetDeviceId();
}

bool UStereoVideoIngestDevice::SetHardwareId_Implementation(const FString& HardwareID)
{
	return false;
}

bool UStereoVideoIngestDevice::Connect_Implementation()
{
	const UStereoVideoIngestDeviceSettings* DeviceSettings = GetSettings();
	FString Path = DeviceSettings->TakeDirectory.Path;

	if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
	{
		SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
		return true;
	}

	return false;
}

bool UStereoVideoIngestDevice::Disconnect_Implementation()
{
	SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);
	return true;
}

bool UStereoVideoIngestDevice::IsVideoFile(const FString& InFileNameWithExtension)
{
	FString Extension = FPaths::GetExtension(InFileNameWithExtension);

	return UE::StereoVideoLiveLinkDevice::Private::SupportedVideoExtensions.Contains(Extension);
}

bool UStereoVideoIngestDevice::IsFrameInSequenceFile(const FString& InFileNameWithExtension)
{
	FString Extension = FPaths::GetExtension(InFileNameWithExtension);

	return UE::StereoVideoLiveLinkDevice::Private::SupportedImageSequenceExtensions.Contains(Extension);
}

bool UStereoVideoIngestDevice::IsAudioFile(const FString& InFileNameWithExtension)
{
	FString Extension = FPaths::GetExtension(InFileNameWithExtension);

	return UE::StereoVideoLiveLinkDevice::Private::SupportedAudioExtensions.Contains(Extension);
}

TOptional<FTakeMetadata> UStereoVideoIngestDevice::CreateTakeMetadata(const FTakeWithComponents& InTakeWithComponents) const
{
	using namespace UE::CaptureManager;

	// Filter video and audio components
	TArray<FTakeWithComponents::Component> VideoComponents = InTakeWithComponents.Components.FilterByPredicate([](const FTakeWithComponents::Component& Component)
	{
		return Component.Type == ETakeComponentType::VIDEO || Component.Type == ETakeComponentType::IMAGE_SEQUENCE;
	});

	check(VideoComponents.Num() == UE::StereoVideoLiveLinkDevice::Private::MaximumVideoFilesCountPerTake);

	TArray<FTakeWithComponents::Component> AudioComponents = InTakeWithComponents.Components.FilterByPredicate([](const FTakeWithComponents::Component& Component)
	{
		return Component.Type == ETakeComponentType::AUDIO;
	});

	// Map component paths to extractor parameters
	const FString& VideoPathA = VideoComponents[0].Path;
	const FString& VideoPathB = VideoComponents[1].Path;

	int32 TakeNumber = InTakeWithComponents.TakeNumber;
	if (TakeNumber == INDEX_NONE)
	{
		TakeNumber = 1;
	}

	FExtractionConfig ExtractionConfig;
	if (const UCaptureManagerSettings* CaptureManagerSettings = GetDefault<UCaptureManagerSettings>())
	{
		if (CaptureManagerSettings->bEnableThirdPartyEncoder && !CaptureManagerSettings->ThirdPartyEncoder.FilePath.IsEmpty())
		{
			ExtractionConfig.bUseFFprobe = CaptureManagerSettings->bEnableThirdPartyEncoder;
			ExtractionConfig.FFmpegPath = CaptureManagerSettings->ThirdPartyEncoder.FilePath;
		}
	}

	FStereoVideoDescriptor Descriptor;
	Descriptor.VideoPathA = VideoPathA;
	Descriptor.VideoPathB = VideoPathB;
	for (const FTakeWithComponents::Component& AudioComponent : AudioComponents)
	{
		Descriptor.AudioFilePaths.Add(FPaths::ConvertRelativePathToFull(InTakeWithComponents.TakePath, AudioComponent.Path));
	}
	Descriptor.Slate = InTakeWithComponents.SlateName;
	Descriptor.TakeNumber = TakeNumber;

	TValueOrError<FTakeMetadata, EStereoVideoExtractionError> Result = ExtractStereoVideoMetadata(MoveTemp(Descriptor), ExtractionConfig);

	if (Result.HasError())
	{
		UE_LOGF(LogStereoVideoIngestDevice, Error, "Failed to extract stereo metadata for take at '%ls'", *InTakeWithComponents.TakePath);
		return {};
	}

	FTakeMetadata TakeMetadata = Result.StealValue();

	// The Core extractor sets generic Video/Audio names from filesystem paths.
	// Override with names from the device's discovery components.
	for (int32 VideoIndex = 0; VideoIndex < VideoComponents.Num() && VideoIndex < TakeMetadata.Video.Num(); ++VideoIndex)
	{
		FString Name = VideoComponents[VideoIndex].Name;
		if (Name.IsEmpty())
		{
			Name = VideoComponents[VideoIndex].Path.Mid(InTakeWithComponents.TakePath.Len() + 1);
			Name = FPaths::GetCleanFilename(Name);
			SanitizePackagePath(Name, '_');
		}

		if (VideoComponents.FindByPredicate([Name, Path = VideoComponents[VideoIndex].Path](const FTakeWithComponents::Component& InOther)
											{
												// Find component with the same name but different path
												return InOther.Name == Name && InOther.Path != Path;
											}))
		{
			// If it exist it is a duplicate, append index to the name (do not change the component)
			Name += TEXT("_") + FString::FromInt(VideoIndex);
		}

		TakeMetadata.Video[VideoIndex].Name = MoveTemp(Name);

		if (VideoComponents[VideoIndex].Type == ETakeComponentType::VIDEO)
		{
			if (!TakeMetadata.Video[VideoIndex].TimecodeStart.IsSet())
			{
				UE_LOGF(LogStereoVideoIngestDevice, Warning, "Failed to determine the timecode for the video file %ls.", *VideoComponents[VideoIndex].Path);
			}

			if (FMath::IsNearlyZero(TakeMetadata.Video[VideoIndex].FrameRate))
			{
				UE_LOGF(LogStereoVideoIngestDevice, Warning, "Failed to determine frame rate for video file: %ls", *VideoComponents[VideoIndex].Path);
			}
		}
	}

	// Override Audio.Name with the device's component-derived names
	int32 AudioNameCounter = 1;
	for (int32 AudioIndex = 0; AudioIndex < AudioComponents.Num() && AudioIndex < TakeMetadata.Audio.Num(); ++AudioIndex)
	{
		FString AudioName = AudioComponents[AudioIndex].Name;
		if (AudioName.IsEmpty())
		{
			AudioName = TEXT("audio") + FString::FromInt(AudioNameCounter);
		}
		++AudioNameCounter;
		TakeMetadata.Audio[AudioIndex].Name = MoveTemp(AudioName);
	}

	// Thumbnail extraction is device-specific (not part of the Core extractor)
	for (int32 VideoIndex = 0; VideoIndex < VideoComponents.Num(); ++VideoIndex)
	{
		if (VideoComponents[VideoIndex].Type == ETakeComponentType::VIDEO)
		{
			FVideoDeviceThumbnailExtractor ThumbnailExtractor;
			TOptional<FTakeThumbnailData::FRawImage> RawImageOpt = ThumbnailExtractor.ExtractThumbnail(VideoComponents[VideoIndex].Path);
			if (RawImageOpt.IsSet())
			{
				TakeMetadata.Thumbnail = MoveTemp(RawImageOpt.GetValue());
				break;
			}
		}
		else // IMAGE_SEQUENCE
		{
			bool bFound = false;
			IFileManager::Get().IterateDirectoryRecursively(*VideoComponents[VideoIndex].Path, [&](const TCHAR* InPath, bool bInIsDirectory) -> bool
			{
				if (!bInIsDirectory)
				{
					TArray<uint8> ThumbnailData;
					if (FFileHelper::LoadFileToArray(ThumbnailData, InPath))
					{
						TakeMetadata.Thumbnail = ThumbnailData;
						bFound = true;
					}
					return false;
				}
				return true;
			});
			if (bFound)
			{
				break;
			}
		}
	}

	return TakeMetadata;
}

#undef LOCTEXT_NAMESPACE
