// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "Customizations/TakeDiscoveryExpressionCustomization.h"

#include "Misc/Optional.h"

#include "Async/TaskProgress.h"

#include "BaseIngestLiveLinkDevice.h"

#include "StereoVideoIngestDevice.generated.h"


UCLASS(BlueprintType)
class UStereoVideoIngestDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:
	UStereoVideoIngestDeviceSettings()
	{
		VideoDiscoveryExpression = TEXT("<Auto>");
		AudioDiscoveryExpression = TEXT("<Auto>");

		DisplayName = TEXT("Stereo Video Ingest");
	}

	/* Path to a directory containing the take(s) data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Video Ingest")
	FDirectoryPath TakeDirectory;

	/**
	 * Format of the file name that the device expects
	 *		Available expressions:
	 *			* <Slate> - slate name
	 *			* <Name> - identifier for the camera in the stereo pair
	 *			* <Take> - take number
	 *			* <Any> - any string
	 *			* <Auto> - used alone to automatically figure out takes based on directory structure
	 *		Available delimiters: '_', '-', '.', '\'
	 *
	 * Examples:
	 * Considering input video MyTakeFolder/MySlate_MyName_SomeString-005.mov
	 *
	 * Video File Name Pattern <Auto> - will set Slate to the containing folder “MyTakeFolder”, Name will be set to the video filename “MySlate_MyName_SomeString-005” and Take will be set to 1 by default
	 *
	 * Video File Name Pattern <Slate>_<Name>_<Any>-<Take> - will extract Slate as “MySlate”, Name as “MyName” and Take as 5. <Any> ensures that the “SomeString” part of the filename is ignored.
	 *
	 * In both cases, extension '.mov' is stripped out.
	 * 
	 * Note: when not using <Auto> expression, then both <Slate> and <Name> must be present in the pattern
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Video Ingest")
	FTakeDiscoveryExpression VideoDiscoveryExpression;

	/**
	 * Format of the file name that the device expects
	 *		Available expressions:
	 *			* <Slate> - slate name
	 *			* <Name> - identifier for the camera in the stereo pair
	 *			* <Take> - take number
	 *			* <Any> - any string
	 *			* <Auto> - used alone to automatically figure out takes based on directory structure
	 *		Available delimiters: '_', '-', '.', '\'
	 *
	 * Examples:
	 * Considering input audio MyTakeFolder/MySlate_MyName_SomeString-005.wav
	 * 
	  * Audio File Name Pattern <Auto> - will set Slate to the containing folder “MyTakeFolder”, Name will be set to the audio filename “MySlate_MyName_SomeString-005” and Take will be set to 1 by default
	 *
	 * Audio File Name Pattern <Slate>_<Name>_<Any>-<Take> - will extract Slate as “MySlate”, Name as “MyName” and Take as 5. <Any> ensures that the “SomeString” part of the filename is ignored.
	 *
	 * In both cases, extension '.wav' is stripped out.
	 * 
	 * Note: when not using <Auto> expression, then both <Slate> and <Name> must be present in the pattern
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Video Ingest")
	FTakeDiscoveryExpression AudioDiscoveryExpression;
};


UCLASS(BlueprintType, meta = (DisplayName = "Stereo Video Ingest", ToolTip = "Ingest subfolders containing pairs of video files as stereo takes"))
class UStereoVideoIngestDevice final : public UBaseIngestLiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
{
	GENERATED_BODY()

public:
	UStereoVideoIngestDevice() = default;
	~UStereoVideoIngestDevice() = default;

	UFUNCTION(BlueprintCallable, Category = "GenericArchiveDeviceSettings")
	const UStereoVideoIngestDeviceSettings* GetSettings() const;

	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;

	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

private:

	enum class ETakeComponentType
	{
		VIDEO,
		IMAGE_SEQUENCE,
		AUDIO
	};

	struct FTakeWithComponents
	{
		struct Component
		{
			FString Name;
			ETakeComponentType Type;
			FString Path;
		};

		FString TakePath;

		FString SlateName;
		int32 TakeNumber;

		TArray<Component> Components;

		int32 CountComponents(ETakeComponentType Type);
	};
	
	using FTakesWithComponents = TArray<struct FTakeWithComponents>;

	virtual FString GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const override;

	//~ Begin ULiveLinkDeviceCapability_Ingest interface
	virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle,
										 const UIngestCapability_Options* InIngestOptions) override;
	virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) override;
	//~ End ULiveLinkDeviceCapability_Ingest interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	bool IsVideoFile(const FString& InFileNameWithExtension);
	bool IsFrameInSequenceFile(const FString& InFileNameWithExtension);
	bool IsAudioFile(const FString& InFileNameWithExtension);

	TOptional<FTakeWithComponents> ExtractTakeComponentsUsingTokens(FString ComponentPath, FString StoragePath, FString Format, ETakeComponentType ComponentType);
	TOptional<FTakeWithComponents> ExtractTakeComponentsFromDirectoryStructure(FString ComponentPath, FString StoragePath, ETakeComponentType ComponentType);
	void GroupFoundComponents(TMap<FString, UStereoVideoIngestDevice::FTakeWithComponents>& TakesWithComponentsCandidates, FTakeWithComponents TakeWithComponents);
	void ExtractTakeComponents(FString ComponentPath, FString StoragePath, ETakeComponentType ComponentType, FString Format, FString UnknownComponentName, TMap<FString, UStereoVideoIngestDevice::FTakeWithComponents>& OutTakesWithComponentsCandidates);
	TMap<FString, FTakeWithComponents> DiscoverTakes(FString StoragePath);
	TOptional<FTakeMetadata> CreateTakeMetadata(const FTakeWithComponents & InTakesWithComponents) const;

	TMap<int32, FString> FullTakePaths;
};
