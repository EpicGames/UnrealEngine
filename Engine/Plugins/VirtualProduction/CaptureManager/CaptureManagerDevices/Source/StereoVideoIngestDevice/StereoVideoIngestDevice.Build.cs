// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class StereoVideoIngestDevice : ModuleRules
{
	public StereoVideoIngestDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "StereoVideoIngest";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Engine",
			"Json",
			"CaptureManagerMediaRW",
			"IngestLiveLinkDevice",
			"DataIngestCore",
			"LiveLinkDevice",
			"LiveLinkCapabilities",
			"Media",
			"LiveLinkHubCaptureMessaging",
			"LiveLinkHub",
			"ToolWidgets",
			"Slate",
			"SlateCore",
			"CaptureManagerTakeMetadata",
			"CaptureManagerSettings",
			"CoreUObject",
			"UnrealEd",
			"VideoLiveLinkDeviceCommon",
			"CaptureMetadataExtraction"
		});

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
