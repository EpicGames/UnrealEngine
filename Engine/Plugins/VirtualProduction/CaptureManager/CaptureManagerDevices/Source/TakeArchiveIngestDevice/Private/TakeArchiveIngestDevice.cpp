// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeArchiveIngestDevice.h"

#include "HAL/FileManager.h"
#include "Settings/CaptureManagerSettings.h"

#include "Utils/TakeArchiveIngestDeviceLog.h"

#include "TakeArchiveMetadataExtractor.h"

#include "Utils/IngestLiveLinkDeviceUtils.h"
#include "Utils/ParseTakeUtils.h"

DEFINE_LOG_CATEGORY(LogTakeArchiveIngestDevice);

const UTakeArchiveIngestDeviceSettings* UTakeArchiveIngestDevice::GetSettings() const
{
	return GetDeviceSettings<UTakeArchiveIngestDeviceSettings>();
}

TSubclassOf<ULiveLinkDeviceSettings> UTakeArchiveIngestDevice::GetSettingsClass() const
{
	return UTakeArchiveIngestDeviceSettings::StaticClass();
}

EDeviceHealth UTakeArchiveIngestDevice::GetDeviceHealth() const
{
	return EDeviceHealth::Nominal;
}

FText UTakeArchiveIngestDevice::GetHealthText() const
{
	return FText::FromString("Nominal");
}

void UTakeArchiveIngestDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::OnSettingChanged(InPropertyChangedEvent);

	if (GET_MEMBER_NAME_CHECKED(UTakeArchiveIngestDeviceSettings, TakeDirectory) == InPropertyChangedEvent.GetMemberPropertyName())
	{
		SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);

		FString Path = GetSettings()->TakeDirectory.Path;
		if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
		{
			SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
		}
		else
		{
			SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);
		}
	}
}

FString UTakeArchiveIngestDevice::GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const
{
	FString StoragePath = GetSettings()->TakeDirectory.Path;

	if (const FString* RelativeTakePath = RelativeTakePaths.Find(InTakeId))
	{
		return FPaths::Combine(StoragePath, *RelativeTakePath);
	}

	return FString();
}

void UTakeArchiveIngestDevice::RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
	auto Task = [WeakThis = TWeakObjectPtr<UTakeArchiveIngestDevice>(this),
		ProcessHandle = TStrongObjectPtr<const UIngestCapability_ProcessHandle>(InProcessHandle),
		IngestOptions = TStrongObjectPtr<const UIngestCapability_Options>(InIngestOptions)]()
		{
			TStrongObjectPtr<UTakeArchiveIngestDevice> This = WeakThis.Pin();

			if (!This.IsValid())
			{
				UE_LOGF(LogTakeArchiveIngestDevice, Warning, "Failed to ingest take, the device has been destroyed");
				return;
			}

			static constexpr uint32 NumberOfTasks = 2; // Convert, Upload

			using namespace UE::CaptureManager;
			TSharedPtr<FTaskProgress> TaskProgress
				= MakeShared<FTaskProgress>(NumberOfTasks, FTaskProgress::FProgressReporter::CreateLambda([WeakThis = WeakThis, ProcessHandle](double InProgress)
					{
						TStrongObjectPtr<UTakeArchiveIngestDevice> This = WeakThis.Pin();

						if (!This.IsValid())
						{
							// Don't log this one, just because it's called so frequently
							return;
						}

						This->ExecuteProcessProgressReporter(ProcessHandle.Get(), InProgress);
					}));

			This->Super::IngestTake(ProcessHandle.Get(), IngestOptions.Get(), MoveTemp(TaskProgress));
		};

	UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Task), UE::Tasks::ETaskPriority::BackgroundNormal);
}

void UTakeArchiveIngestDevice::RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback)
{
	using namespace UE::CaptureManager;

	RemoveAllTakes();

	FString StoragePath = GetSettings()->TakeDirectory.Path;

	TArray<FString> TakeMetadataFiles;
	const bool bIterationResult = IFileManager::Get().IterateDirectoryRecursively(*StoragePath, [&](const TCHAR* InFileNameOrDirectory, bool bInIsDirectory)
		{
			if (IsUpdateTakeListAbortRequested())
			{
				return false;
			}

			if (!bInIsDirectory)
			{
				const FString CurrentFileName = FPaths::GetCleanFilename(InFileNameOrDirectory);
				if (CurrentFileName == TEXT("take.json"))
				{
					TakeMetadataFiles.Add(InFileNameOrDirectory);
				}
				else if (FPaths::GetExtension(CurrentFileName) == FTakeMetadata::FileExtension)
				{
					TakeMetadataFiles.Add(InFileNameOrDirectory);
				}
			}

			return true;
		});

	FExtractionConfig ExtractionConfig;

	if (const UCaptureManagerSettings* CaptureManagerSettings = GetDefault<UCaptureManagerSettings>())
	{
		if (CaptureManagerSettings->bEnableThirdPartyEncoder && !CaptureManagerSettings->ThirdPartyEncoder.FilePath.IsEmpty())
		{
			ExtractionConfig.bUseFFprobe = CaptureManagerSettings->bEnableThirdPartyEncoder;
			ExtractionConfig.FFmpegPath = CaptureManagerSettings->ThirdPartyEncoder.FilePath;
		}
	}

	for (const FString& CurrentTakeFile : TakeMetadataFiles)
	{
		if (IsUpdateTakeListAbortRequested())
		{
			ExecuteUpdateTakeListCallback(InCallback, TArray<int32>());
			return;
		}

		TValueOrError<FTakeMetadata, ETakeArchiveExtractionError> TakeMetadata = ExtractTakeArchiveMetadata(CurrentTakeFile, ExtractionConfig);

		if (TakeMetadata.HasValue())
		{
			int32 TakeId = AddTake(TakeMetadata.StealValue());
			FString CurrentDirectory = FPaths::GetPath(CurrentTakeFile);

			FString TakePath = CurrentDirectory.Right(CurrentDirectory.Len() - StoragePath.Len());
			FPaths::NormalizeDirectoryName(TakePath); // Removes trailing slash
			RelativeTakePaths.Add(TakeId, MoveTemp(TakePath));

			PublishEvent<FIngestCapability_TakeAddedEvent>(TakeId);
		}
	}

	ExecuteUpdateTakeListCallback(InCallback, Execute_GetTakeIdentifiers(this));
}

ELiveLinkDeviceConnectionStatus UTakeArchiveIngestDevice::GetConnectionStatus_Implementation() const
{
	const UTakeArchiveIngestDeviceSettings* DeviceSettings = GetSettings();
	FString Path = DeviceSettings->TakeDirectory.Path;

	if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
	{
		return ELiveLinkDeviceConnectionStatus::Connected;
	}

	return ELiveLinkDeviceConnectionStatus::Disconnected;
}

FString UTakeArchiveIngestDevice::GetHardwareId_Implementation() const
{
	return FPlatformMisc::GetDeviceId();
}

bool UTakeArchiveIngestDevice::SetHardwareId_Implementation(const FString& HardwareID)
{
	return false;
}

bool UTakeArchiveIngestDevice::Connect_Implementation()
{
	const UTakeArchiveIngestDeviceSettings* DeviceSettings = GetSettings();
	FString Path = DeviceSettings->TakeDirectory.Path;

	if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
	{
		SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
		return true;
	}
	
	return false;
}

bool UTakeArchiveIngestDevice::Disconnect_Implementation()
{
	SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);
	return true;
}

