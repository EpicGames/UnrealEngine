// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "LiveLinkDeviceCapability_Connection.h"

#include "Misc/Optional.h"

#include "Async/TaskProgress.h"

#include "BaseIngestLiveLinkDevice.h"

#include "TakeArchiveIngestDevice.generated.h"


UCLASS(BlueprintType)
class UTakeArchiveIngestDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:

	UTakeArchiveIngestDeviceSettings()
	{
		DisplayName = TEXT("Take Archive Ingest");
	}

	/* Path to a directory containing the take(s) data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Take Archive Ingest")
	FDirectoryPath TakeDirectory;
};


UCLASS(BlueprintType, meta = (DisplayName = "Take Archive Ingest", ToolTip = "Ingest take archives (described by .cptake files) and legacy Capture Manager takes"))
class UTakeArchiveIngestDevice final : public UBaseIngestLiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "TakeArchiveIngestDeviceSettings")
	const UTakeArchiveIngestDeviceSettings* GetSettings() const;

	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;

	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

private:
	virtual FString GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const override;

	//~ Begin ULiveLinkDeviceCapability_Ingest interface
	virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) override;
	virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) override;
	//~ End ULiveLinkDeviceCapability_Ingest interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	TMap<int32, FString> RelativeTakePaths;
};
