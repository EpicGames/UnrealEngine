// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class TakeArchiveIngestDevice : ModuleRules
{
	public TakeArchiveIngestDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "TakeArchiveIngest";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Engine",
			"Json",
			"CaptureManagerMediaRW",
			"IngestLiveLinkDevice",
			"DataIngestCore",
			"LiveLinkDevice",
			"LiveLinkCapabilities",
			"Media",
			"LiveLinkHubCaptureMessaging",
			"LiveLinkHub",
			"ToolWidgets",
			"Slate",
			"SlateCore",
			"CaptureManagerTakeMetadata",
			"CaptureManagerSettings",
			"CaptureMetadataExtraction",
			"CoreUObject",
			"UnrealEd"
		});

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
