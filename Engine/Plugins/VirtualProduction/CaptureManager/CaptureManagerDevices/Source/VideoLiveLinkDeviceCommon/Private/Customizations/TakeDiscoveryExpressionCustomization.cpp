// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/TakeDiscoveryExpressionCustomization.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "DiscoveryExpressionCustomization"

FTakeDiscoveryExpressionCustomization::FTakeDiscoveryExpressionCustomization()
{
}

void FTakeDiscoveryExpressionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FTakeDiscoveryExpressionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	ExpressionProperty = PropertyHandle;

	ChildBuilder.AddProperty(PropertyHandle)
		.CustomWidget()
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
				.Text(this, &FTakeDiscoveryExpressionCustomization::OnGetExpressionValue)
				.OnVerifyTextChanged(this, &FTakeDiscoveryExpressionCustomization::OnExpressionValidate)
				.OnTextCommitted(this, &FTakeDiscoveryExpressionCustomization::OnExpressionCommited)
				.SelectAllTextWhenFocused(true)
				.RevertTextOnEscape(true)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsReadOnly(this, &FTakeDiscoveryExpressionCustomization::IsReadOnly)
		];
}

TOptional<FText> FTakeDiscoveryExpressionCustomization::ValidateExpression(const FString& InExpression)
{
	if (InExpression.IsEmpty())
	{
		return LOCTEXT("Discovery_Expression_Empty", "Expression field must not be empty");
	}

	if (InExpression == "<Auto>")
	{
		return {};
	}

	TArray<FString> MandatoryExpressionParts = { "<Slate>", "<Name>" };

	for (const FString& ExpressionPart : MandatoryExpressionParts)
	{
		int32 FoundIndex = InExpression.Find(ExpressionPart);
		if (FoundIndex == INDEX_NONE)
		{
			return FText::Format(LOCTEXT("Discovery_Expression_Missing_Mandatory_Part", "Missing mandatory part: {0}"), FText::FromString(ExpressionPart));
		}

		FoundIndex = InExpression.Find(ExpressionPart, ESearchCase::CaseSensitive, ESearchDir::FromStart, FoundIndex + 1);
		if (FoundIndex != INDEX_NONE)
		{
			return FText::Format(LOCTEXT("Discovery_Expression_Part_Repetition", "Multiple expression parts: {0}"), FText::FromString(ExpressionPart));
		}
	}

	TArray<FString> OptionalExpressionParts = { "<Take>" };

	for (const FString& ExpressionPart : OptionalExpressionParts)
	{
		int32 FoundIndex = InExpression.Find(ExpressionPart);
		if (FoundIndex != INDEX_NONE)
		{
			FoundIndex = InExpression.Find(ExpressionPart, ESearchCase::CaseSensitive, ESearchDir::FromStart, FoundIndex + 1);

			if (FoundIndex != INDEX_NONE)
			{
				return FText::Format(LOCTEXT("Discovery_Expression_Part_Repetition", "Multiple expression parts: {0}"), FText::FromString(ExpressionPart));
			}
		}
	}

	return {};
}

FText FTakeDiscoveryExpressionCustomization::OnGetExpressionValue() const
{
	TArray<void*> RawValue;
	ExpressionProperty->AccessRawData(RawValue);

	if (RawValue.Num())
	{
		const FTakeDiscoveryExpression* Expression = static_cast<const FTakeDiscoveryExpression*>(RawValue[0]);
		return FText::FromString(Expression->Value);
	}

	return FText::GetEmpty();
}

bool FTakeDiscoveryExpressionCustomization::OnExpressionValidate(const FText& InText, FText& OutErrorText)
{
	FString Value = InText.ToString();

	TOptional<FText> MaybeError = ValidateExpression(InText.ToString());

	if (MaybeError.IsSet())
	{
		OutErrorText = MaybeError.GetValue();
		return false;
	}

	return true;
}

void FTakeDiscoveryExpressionCustomization::OnExpressionCommited(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::Type::OnCleared)
	{
		return;
	}

	TArray<void*> RawValue;
	ExpressionProperty->AccessRawData(RawValue);

	FString Value = InText.ToString();

	if (RawValue.Num())
	{
		GEditor->BeginTransaction(FText::Format(LOCTEXT("DiscoveryExpression_SetProperty", "Edit {0}"), ExpressionProperty->GetPropertyDisplayName()));

		ExpressionProperty->NotifyPreChange();

		FTakeDiscoveryExpression* TakesDiscoveryTokesPtr = static_cast<FTakeDiscoveryExpression*>(RawValue[0]);
		TakesDiscoveryTokesPtr->Value = MoveTemp(Value);

		ExpressionProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		ExpressionProperty->NotifyFinishedChangingProperties();

		GEditor->EndTransaction();
	}
}

bool FTakeDiscoveryExpressionCustomization::IsReadOnly() const
{
	return !ExpressionProperty->IsEditable();
}

#undef LOCTEXT_NAMESPACE