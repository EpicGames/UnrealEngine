// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/TakeDiscoveryExpressionParser.h"

#include "Misc/AutomationTest.h"

static const TArray<FString::ElementType> Delimiters =
{
	'-',
	'_',
	'.',
	'/'
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest1, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Success",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest1::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>_<Take>_<Name>");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { SlateName, TakeNumber, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());
	TestEqual(TEXT("Take Number matches"), TakeNumber, Parser.GetTakeNumber());
	TestEqual(TEXT("Name matches"), Name, Parser.GetName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest2, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Fail",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest2::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Take>_<Name>");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { SlateName, TakeNumber, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest3, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Postfix.SuccessAny",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest3::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>_<Name>_<Any>");

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}_TestingAnythingHere"), { SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());
	TestEqual(TEXT("Name matches"), Name, Parser.GetName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest4, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Postfix.FailAny",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest4::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>_<Name>_");

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}_TestingAnythingHere"), { SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest5, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Postfix.SuccessCustom",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest5::RunTest(const FString& Parameters)
{
	FString CustomString = TEXT("CustomString");

	FString InputFormat = FString::Format(TEXT("<Slate>_<Name>_{0}"), { CustomString });

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { SlateName, Name, CustomString });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());
	TestEqual(TEXT("Name matches"), Name, Parser.GetName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest6, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Postfix.FailCustom",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest6::RunTest(const FString& Parameters)
{
	FString CustomString = TEXT("CustomString");

	FString InputFormat = FString::Format(TEXT("<Slate>_<Name>_{0}"), { CustomString });

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString InvalidCustomString = TEXT("InvalidCustomString");
	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { SlateName, Name, InvalidCustomString });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse fail"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest7, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Prefix.SuccessAny",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest7::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Any>_<Slate>_<Name>");

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("TestingAnythingHere_{0}_{1}"), { SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());
	TestEqual(TEXT("Name matches"), Name, Parser.GetName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest8, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Prefix.FailAny",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest8::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("_<Slate>_<Name>");

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("TestingAnythingHere_{0}_{1}"), { SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest9, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Prefix.SuccessCustom",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest9::RunTest(const FString& Parameters)
{
	FString CustomString = TEXT("CustomString");

	FString InputFormat = FString::Format(TEXT("{0}_<Slate>_<Name>"), { CustomString });

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { CustomString, SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());
	TestEqual(TEXT("Name matches"), Name, Parser.GetName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest10, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Prefix.FailCustom",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest10::RunTest(const FString& Parameters)
{
	FString CustomString = TEXT("CustomString");

	FString InputFormat = FString::Format(TEXT("{0}_<Slate>_<Name>"), { CustomString });

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString InvalidCustomString = TEXT("InvalidCustomString");
	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { InvalidCustomString, SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse fail"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest11, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Mid.SuccessAny",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest11::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>_<Any>_<Name>");

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_TestingAnythingHere_{1}"), { SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());
	TestEqual(TEXT("Name matches"), Name, Parser.GetName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest12, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Mid.FailAny",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest12::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>_<Name>");

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_TestingAnythingHere_{1}"), { SlateName, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest13, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Mid.SuccessCustom",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest13::RunTest(const FString& Parameters)
{
	FString CustomString = TEXT("CustomString");

	FString InputFormat = FString::Format(TEXT("<Slate>_{0}_<Name>"), { CustomString });

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { SlateName, CustomString, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());
	TestEqual(TEXT("Name matches"), Name, Parser.GetName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest14, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.Mid.FailCustom",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest14::RunTest(const FString& Parameters)
{
	FString CustomString = TEXT("CustomString");

	FString InputFormat = FString::Format(TEXT("<Slate>_{0}_<Name>"), { CustomString });

	FString SlateName = TEXT("SlateName");
	FString Name = TEXT("Name");

	FString InvalidCustomString = TEXT("InvalidCustomString");
	FString FormattedValue = FString::Format(TEXT("{0}_{1}_{2}"), { SlateName, InvalidCustomString, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse fail"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest15, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.DifferentDelimiterSuccess",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest15::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>_<Take>-<Name>");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}-{2}"), { SlateName, TakeNumber, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest16, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.DifferentDelimiterFailure",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest16::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("{Slate}-{Name}");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}_{1}-{2}"), { SlateName, TakeNumber, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());
	
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest17, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.SingleTokenValue",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest17::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;

	FString FormattedValue = FString::Format(TEXT("{0}"), { SlateName });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestEqual(TEXT("Slate Name matches"), SlateName, Parser.GetSlateName());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest18, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.SingleValue",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest18::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("TestingString");
	FString TestingString = TEXT("TestingString");

	int32 NoTakeNumber = INDEX_NONE;

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(TestingString), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());
	TestTrue(TEXT("Slate Name empty"), Parser.GetSlateName().IsEmpty());
	TestEqual(TEXT("Take Number invalid"), NoTakeNumber, Parser.GetTakeNumber());
	TestTrue(TEXT("Name empty"), Parser.GetName().IsEmpty());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest19, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.EmptyString",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest19::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;

	FString FormattedValue = FString::Format(TEXT("{0}_{1}"), { SlateName, TakeNumber });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest20, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.MultipleDelimiters",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest20::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("______");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;

	FString FormattedValue = FString::Format(TEXT("{0}_{1}"), { SlateName, TakeNumber });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest21, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.SingleDelimiter",
								 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest21::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("_");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;

	FString FormattedValue = FString::Format(TEXT("{0}_{1}"), { SlateName, TakeNumber });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestFalse(TEXT("Parse failed"), Parser.Parse());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTakeDiscoveryExpressionParserTest22, "CaptureManager.VideoLiveLinkDeviceCommon.DiscoveryExpressionParser.DifferentDelimiter2Success",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FTakeDiscoveryExpressionParserTest22::RunTest(const FString& Parameters)
{
	FString InputFormat = TEXT("<Slate>/<Take>-<Name>");

	FString SlateName = TEXT("SlateName");
	int32 TakeNumber = 10;
	FString Name = TEXT("Name");

	FString FormattedValue = FString::Format(TEXT("{0}/{1}-{2}"), { SlateName, TakeNumber, Name });

	FTakeDiscoveryExpressionParser Parser(MoveTemp(InputFormat), MoveTemp(FormattedValue), Delimiters);
	TestTrue(TEXT("Parse succeeded"), Parser.Parse());

	return true;
}
