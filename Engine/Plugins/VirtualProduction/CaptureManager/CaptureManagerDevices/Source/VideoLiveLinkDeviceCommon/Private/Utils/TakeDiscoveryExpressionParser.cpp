// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/TakeDiscoveryExpressionParser.h"

#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Containers/StringView.h"

const FString FTakeDiscoveryExpressionParser::SlateNameToken = TEXT("<Slate>");
const FString FTakeDiscoveryExpressionParser::TakeNumberToken = TEXT("<Take>");
const FString FTakeDiscoveryExpressionParser::NameToken = TEXT("<Name>");
const FString FTakeDiscoveryExpressionParser::AnyToken = TEXT("<Any>");

FTakeDiscoveryExpressionParser::FTakeDiscoveryExpressionParser(const FString& InFormat, const FString& InFormattedValue, const TArray<FString::ElementType>& InDelimiters)
	: Format(InFormat)
	, FormattedValue(InFormattedValue)
	, AllowedDelimiters(InDelimiters)
{
}

bool FTakeDiscoveryExpressionParser::Parse()
{
	// Figure out which delimiters user actually uses in specified format
	TArray<FString::ElementType> FoundDelimiters;
	for (FString::ElementType AllowedDelimiter : AllowedDelimiters)
	{
		FString AllowedDelimiterStr;
		AllowedDelimiterStr.AppendChar(AllowedDelimiter);
		if (Format.Contains(AllowedDelimiterStr))
		{
			FoundDelimiters.Add(AllowedDelimiter);
		}
	}
	
	TOptional<FComponentValueMap> ComponentValueMap = ParseWithDelimiters(FoundDelimiters);
	
	if (!ComponentValueMap.IsSet())
	{
		return false;
	}

	return DetermineValues(MoveTemp(ComponentValueMap.GetValue()));
}

TOptional<FTakeDiscoveryExpressionParser::FComponentValueMap> FTakeDiscoveryExpressionParser::ParseWithDelimiters(TArray<FString::ElementType> InDelimiters) const
{
	TArray<FString> FormatComponents = ParseWithDelimiters(Format, InDelimiters);

	if (FormatComponents.IsEmpty())
	{
		return {};
	}

	TArray<FString> FileNameComponents = ParseWithDelimiters(FormattedValue, InDelimiters);

	if (FileNameComponents.IsEmpty())
	{
		return {};
	}

	if (FormatComponents.Num() != FileNameComponents.Num())
	{
		return {};
	}

	FComponentValueMap ComponentValueMap;
	for (int32 Index = 0; Index < FileNameComponents.Num(); ++Index)
	{
		ComponentValueMap.Add(FormatComponents[Index], FileNameComponents[Index]);
	}

	return ComponentValueMap;
}

TArray<FString> FTakeDiscoveryExpressionParser::ParseWithDelimiters(FString InString, const TArray<FString::ElementType>& InDelimiters, int32 InDelimiterIndex) const
{
	TArray<FString> Result;
	if (InDelimiterIndex >= InDelimiters.Num())
	{
		Result.Add(InString);
		return Result;
	}

	FString Delimiter;
	Delimiter.AppendChar(InDelimiters[InDelimiterIndex]);

	TArray<FString> HalfwaySplitComponents;

	InString.ParseIntoArray(HalfwaySplitComponents, *Delimiter);

	for (const FString& Component : HalfwaySplitComponents)
	{
		for (const FString& Part : ParseWithDelimiters(Component, InDelimiters, InDelimiterIndex + 1))
		{
			Result.Add(Part);
		}
	}
	
	return Result;
}


TOptional<FTakeDiscoveryExpressionParser::FComponentValueMap> FTakeDiscoveryExpressionParser::ParseWithoutDelimiters() const
{
	FComponentValueMap ComponentValueMap;
	ComponentValueMap.Add(Format, FormattedValue);

	return ComponentValueMap;
}

bool FTakeDiscoveryExpressionParser::DetermineValues(FComponentValueMap InComponentValueMap)
{
	if (const FString* MaybeSlateName = InComponentValueMap.Find(SlateNameToken))
	{
		SlateName = *MaybeSlateName;
		InComponentValueMap.Remove(SlateNameToken);
	}

	if (const FString* MaybeTakeNumber = InComponentValueMap.Find(TakeNumberToken))
	{
		TakeNumber = FCString::Atoi(**MaybeTakeNumber);
		InComponentValueMap.Remove(TakeNumberToken);
	}

	if (const FString* MaybeName = InComponentValueMap.Find(NameToken))
	{
		Name = *MaybeName;
		InComponentValueMap.Remove(NameToken);
	}

	InComponentValueMap.Remove(AnyToken);

	for (const TPair<FString, FString>& AdditionalComponentValue : InComponentValueMap)
	{
		if (AdditionalComponentValue.Key != AdditionalComponentValue.Value)
		{
			return false;
		}
	}

	return true;
}

FString FTakeDiscoveryExpressionParser::GetSlateName() const
{
	return SlateName;
}

int32 FTakeDiscoveryExpressionParser::GetTakeNumber() const
{
	return TakeNumber;
}

FString FTakeDiscoveryExpressionParser::GetName() const
{
	return Name;
}
