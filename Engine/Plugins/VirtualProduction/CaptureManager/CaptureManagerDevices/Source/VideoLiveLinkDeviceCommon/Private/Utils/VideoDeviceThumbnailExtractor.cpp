// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/VideoDeviceThumbnailExtractor.h"
#include "ImageUtils.h"
#include "MediaSample.h"
#include "Engine/Texture2D.h"
#include "CaptureManagerMediaRWModule.h"
#include "Serialization/JsonSerializer.h"

#include "Settings/CaptureManagerSettings.h"
#include "ProcessRunner/ProcessRunner.h"

#include "Utils/MediaPixelFormatConversions.h"

DEFINE_LOG_CATEGORY_STATIC(LogVideoDeviceThumbnailExtractor, Log, All);

namespace UE::CaptureManager
{

FVideoDeviceThumbnailExtractor::FVideoDeviceThumbnailExtractor() = default;

TOptional<FTakeThumbnailData::FRawImage> FVideoDeviceThumbnailExtractor::ExtractThumbnail(const FString& InCurrentFile)
{
	FMediaRWManager& MediaManager = FModuleManager::LoadModuleChecked<FCaptureManagerMediaRWModule>("CaptureManagerMediaRW").Get();

	TArray<FColor> Thumbnail;
	int32 Width = 0;
	int32 Height = 0;

	// If the third party encoder is enabled, then always use ffmpeg to gather thumbnail data.
	const UCaptureManagerSettings* Settings = GetDefault<UCaptureManagerSettings>();
	if (Settings->bEnableThirdPartyEncoder)
	{
		Thumbnail = ObtainThumbnailFromThirdPartyEncoder(Settings->ThirdPartyEncoder.FilePath, InCurrentFile);

		if (Thumbnail.IsEmpty())
		{
			return {};
		}

		// As we specified the width during image creation we can derive
		// the height from the final number of pixels in the image.
		Width = ThirdPartyEncoderThumbnailWidth;
		Height = Thumbnail.Num() / Width;
	}
	else // Otherwise attempt to use an appropriate video reader for the file 
	{
		TValueOrError<TUniquePtr<IVideoReader>, FText> VideoReaderResult = MediaManager.CreateVideoReader(InCurrentFile);
		if (VideoReaderResult.HasValue())
		{
			TUniquePtr<IVideoReader> VideoReader = VideoReaderResult.StealValue();
			TValueOrError<TUniquePtr<UE::CaptureManager::FMediaTextureSample>, FText> TextureSampleResult = VideoReader->Next();
			if (TextureSampleResult.HasValue())
			{
				const UE::CaptureManager::FMediaTextureSample* Sample = TextureSampleResult.GetValue().Get();

				if (Sample)
				{
					Thumbnail = ConvertThumbnailFromSample(Sample);
					Width = Sample->Dimensions.X;
					Height = Sample->Dimensions.Y;
				}
			}
			else
			{
				FText ErrorText = TextureSampleResult.StealError();
				UE_LOGF(LogVideoDeviceThumbnailExtractor, Warning, "Couldn't obtain the thumbnail from the video %ls : %ls", *InCurrentFile, *ErrorText.ToString());
			}
		}
		else
		{
			FText ErrorText = VideoReaderResult.StealError();
			UE_LOGF(LogVideoDeviceThumbnailExtractor, Warning, "Couldn't open video file %ls : %ls", *InCurrentFile, *ErrorText.ToString());
		}
	}
	
	if (Thumbnail.IsEmpty())
	{
		return {};
	}

	FTakeThumbnailData::FRawImage RawImage;
	RawImage.DecompressedImageData = MoveTemp(Thumbnail);
	RawImage.Width = Width;
	RawImage.Height = Height;
	RawImage.Format = ERawImageFormat::BGRA8;

	return RawImage;
}

TArray<FColor> FVideoDeviceThumbnailExtractor::ConvertThumbnailFromSample(const FMediaTextureSample* InSample)
{
	EMediaTexturePixelFormat SampleFormat = InSample->CurrentFormat;
	const TArray<uint8>& Buffer = InSample->Buffer;

	TArray<FColor> ThumbnailRawColorData;
	switch (SampleFormat)
	{
		case EMediaTexturePixelFormat::U8_Mono:
		{
			ThumbnailRawColorData.Reserve(Buffer.Num());
			for (const uint8& Value : Buffer)
			{
				ThumbnailRawColorData.Add(FColor(Value, Value, Value));
			}
			break;
		}
		case EMediaTexturePixelFormat::U8_RGB:
		{
			const int32 NumChannels = 3;
			check(Buffer.Num() % NumChannels == 0);

			ThumbnailRawColorData.Reserve(Buffer.Num() / NumChannels);
			for (int32 ValueIndex = 0; ValueIndex < Buffer.Num(); ValueIndex += NumChannels)
			{
				ThumbnailRawColorData.Add(FColor(Buffer[ValueIndex], Buffer[ValueIndex + 1], Buffer[ValueIndex + 2]));
			}
			break;
		}
		case EMediaTexturePixelFormat::U8_BGR:
		{
			const int32 NumChannels = 3;
			check(Buffer.Num() % NumChannels == 0);

			ThumbnailRawColorData.Reserve(Buffer.Num() / NumChannels);
			for (int32 ValueIndex = 0; ValueIndex < Buffer.Num(); ValueIndex += NumChannels)
			{
				ThumbnailRawColorData.Add(FColor(Buffer[ValueIndex + 2], Buffer[ValueIndex + 1], Buffer[ValueIndex]));
			}
			break;
		}
		case EMediaTexturePixelFormat::U8_RGBA:
		{
			const int32 NumChannels = 4;
			check(Buffer.Num() % NumChannels == 0);

			ThumbnailRawColorData.Reserve(Buffer.Num() / NumChannels);
			for (int32 ValueIndex = 0; ValueIndex < Buffer.Num(); ValueIndex += NumChannels)
			{
				ThumbnailRawColorData.Add(FColor(Buffer[ValueIndex], Buffer[ValueIndex + 1], Buffer[ValueIndex + 2], Buffer[ValueIndex + 3]));
			}
			break;
		}
		case EMediaTexturePixelFormat::U8_BGRA:
		{
			const int32 NumChannels = 4;
			check(Buffer.Num() % NumChannels == 0);

			ThumbnailRawColorData.Reserve(Buffer.Num() / NumChannels);
			for (int32 ValueIndex = 0; ValueIndex < Buffer.Num(); ValueIndex += NumChannels)
			{
				ThumbnailRawColorData.Add(FColor(Buffer[ValueIndex + 2], Buffer[ValueIndex + 1], Buffer[ValueIndex], Buffer[ValueIndex + 3]));
			}
			break;
		}
		case EMediaTexturePixelFormat::U8_I420:
			ThumbnailRawColorData = UE::CaptureManager::UEConvertI420ToBGRA(InSample);
			break;
		case EMediaTexturePixelFormat::U8_NV12:
			ThumbnailRawColorData = UE::CaptureManager::UEConvertNV12ToBGRA(InSample);
			break;
		case EMediaTexturePixelFormat::U8_YUY2:
			ThumbnailRawColorData = UE::CaptureManager::UEConvertYUY2ToBGRA(InSample);
			break;
		default:
		{
			UE_LOGF(LogVideoDeviceThumbnailExtractor, Warning, "Unsupported image format");
		}
	}

	return ThumbnailRawColorData;
}

TArray<FColor> FVideoDeviceThumbnailExtractor::ObtainThumbnailFromThirdPartyEncoder(const FString& InEncoderPath, const FString& InCurrentFile)
{
	FString ScaleArg = FString::Printf(TEXT("scale=%d:-1"), ThirdPartyEncoderThumbnailWidth);

	FString CommandLineArgs = FString::Format(TEXT("-hide_banner -loglevel error -i \"{0}\" -map v:0 -vf \"thumbnail=2, {1}\" -frames:v 1 -f rawvideo -pix_fmt rgb24 -"), { InCurrentFile, ScaleArg });

	FProcessRunnerResult RunnerResult = FProcessRunner::Run(InEncoderPath, CommandLineArgs);

	if (!RunnerResult.HasValue())
	{
		return TArray<FColor>(); 
	}

	TArray<uint8> ThumbnailRawData = RunnerResult.StealValue();
	int32 ThumbnailNumBytes = ThumbnailRawData.Num();

	constexpr int32 NumChannels = 3;
	if (ThumbnailNumBytes % NumChannels != 0)
	{
		UE_LOGF(LogVideoDeviceThumbnailExtractor, Warning, "Thumbnail raw data is malformed and the byte count is not a multiple of %d. Num bytes is %d", NumChannels, ThumbnailNumBytes);
		return TArray<FColor>();
	}

	TArray<FColor> ThumbnailRawColorData;
	ThumbnailRawColorData.Reserve(ThumbnailNumBytes / NumChannels);

	for (int32 ValueIndex = 0; ValueIndex < ThumbnailNumBytes; ValueIndex += NumChannels)
	{
		ThumbnailRawColorData.Add(FColor(ThumbnailRawData[ValueIndex], ThumbnailRawData[ValueIndex + 1], ThumbnailRawData[ValueIndex + 2]));
	}

	return ThumbnailRawColorData;
}

}
