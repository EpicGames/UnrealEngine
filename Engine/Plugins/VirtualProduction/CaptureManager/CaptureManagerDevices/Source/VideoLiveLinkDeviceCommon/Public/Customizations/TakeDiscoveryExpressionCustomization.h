// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"

#include "TakeDiscoveryExpressionCustomization.generated.h"

USTRUCT(BlueprintType)
struct VIDEOLIVELINKDEVICECOMMON_API FTakeDiscoveryExpression
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Discovert Expression Value")
	FString Value;

	FTakeDiscoveryExpression operator=(const FString& InValue)
	{
		Value = InValue;
		return *this;
	}
};

class VIDEOLIVELINKDEVICECOMMON_API FTakeDiscoveryExpressionCustomization : public IPropertyTypeCustomization
{
public:
	FTakeDiscoveryExpressionCustomization();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	FText OnGetExpressionValue() const;
	bool OnExpressionValidate(const FText& InText, FText& OutErrorText);
	void OnExpressionCommited(const FText& InText, ETextCommit::Type CommitInfo);
	bool IsReadOnly() const;
	TOptional<FText> ValidateExpression(const FString& InTokens);

	TSharedPtr<IPropertyHandle> ExpressionProperty;
};
