// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

class VIDEOLIVELINKDEVICECOMMON_API FTakeDiscoveryExpressionParser
{
public:

	static const FString SlateNameToken;
	static const FString TakeNumberToken;
	static const FString NameToken;
	static const FString AnyToken;

	FTakeDiscoveryExpressionParser(const FString& InFormat, const FString& InFormattedValue, const TArray<FString::ElementType>& InDelimiters);

	bool Parse();

	FString GetSlateName() const;
	int32 GetTakeNumber() const;
	FString GetName() const;

private:

	using FComponentValueMap = TMap<FString, FString>;

	TOptional<FComponentValueMap> ParseWithDelimiters(TArray<FString::ElementType> InDelimiters) const;
	TOptional<FComponentValueMap> ParseWithoutDelimiters() const;
	TArray<FString> ParseWithDelimiters(FString InString, const TArray<FString::ElementType>& InDelimiters, int32 InDelimiterIndex = 0) const;

	bool DetermineValues(FComponentValueMap InComponentValueMap);

	FString Format;
	FString FormattedValue;
	const TArray<FString::ElementType>& AllowedDelimiters;

	FString SlateName;
	int32 TakeNumber = INDEX_NONE;
	FString Name;
};