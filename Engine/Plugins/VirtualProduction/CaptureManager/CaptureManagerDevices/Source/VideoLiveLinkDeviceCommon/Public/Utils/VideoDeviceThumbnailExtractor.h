// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "CaptureManagerTakeMetadata.h"

#include "Misc/CoreMiscDefines.h"

#define UE_API VIDEOLIVELINKDEVICECOMMON_API

namespace UE::CaptureManager 
{
struct FMediaTextureSample;

class FVideoDeviceThumbnailExtractor
{
public:

	UE_INTERNAL UE_API FVideoDeviceThumbnailExtractor();

	UE_INTERNAL UE_API TOptional<FTakeThumbnailData::FRawImage> ExtractThumbnail(const FString& InCurrentFile);

private:

	TArray<FColor> ConvertThumbnailFromSample(const struct FMediaTextureSample* InSample);

	static constexpr int32 ThirdPartyEncoderThumbnailWidth = 256;

	TArray<FColor> ObtainThumbnailFromThirdPartyEncoder(const FString& InEncoderPath, const FString& InCurrentFile);

	TArray<uint8> RunProcess(const FString& InProcessName, const FString& InProcessArgs);
	TArray<uint8> ReadFromPipe(void* InPipe);
};

}

#undef UE_API