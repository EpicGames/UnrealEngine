// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class VideoLiveLinkDeviceCommon : ModuleRules
{
	public VideoLiveLinkDeviceCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "VideoLLDCommon";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureManagerTakeMetadata",
			"ImageCore",
			"Core",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CaptureManagerMediaRW",
			"CaptureManagerSettings",
			"CaptureUtils",
			"Engine",
			"Slate",
			"SlateCore",
			"CoreUObject",
			"UnrealEd"
		});
	}
}
